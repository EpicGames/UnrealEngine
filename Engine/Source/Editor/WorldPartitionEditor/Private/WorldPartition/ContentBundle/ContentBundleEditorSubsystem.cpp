// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/ContentBundle/ContentBundleEditorSubsystem.h"
#include "WorldPartition/ContentBundle/ContentBundleEngineSubsystem.h"
#include "WorldPartition/ContentBundle/ContentBundleWorldSubsystem.h"
#include "WorldPartition/ContentBundle/ContentBundlePaths.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/ContentBundle/ContentBundleEditor.h"
#include "WorldPartition/ContentBundle/ContentBundleLog.h"
#include "WorldPartition/ContentBundle/ContentBundle.h"
#include "WorldPartition/DataLayer/ExternalDataLayerManager.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"
#include "WorldPartition/WorldPartition.h"
#include "GameFramework/ActorPrimitiveColorHandler.h"
#include "Subsystems/ActorEditorContextSubsystem.h"
#include "DataLayer/DataLayerEditorSubsystem.h"
#include "Engine/Selection.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Editor.h"
#include "ActorEditorContextState.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBundleEditorSubsystem)

#define LOCTEXT_NAMESPACE "ContentBundle"

#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
static FName NAME_ContentBundleColor(TEXT("ContentBundleColor"));
#endif

void UContentBundleEditingSubmodule::DoInitialize()
{
	check(GEditor);
	UActorEditorContextSubsystem::Get()->RegisterClient(this);
	// For backward compatibility, Content Bundle will be applied right when LevelActorAdded is called
	GEditor->OnLevelActorAdded().AddUObject(this, &UContentBundleEditingSubmodule::ApplyContext);

#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	FActorPrimitiveColorHandler::Get().RegisterPrimitiveColorHandler(NAME_ContentBundleColor, LOCTEXT("ContentBundleColor", "Content Bundle Color"), [](const UPrimitiveComponent* InPrimitiveComponent) -> FLinearColor
	{
		if (AActor* Actor = InPrimitiveComponent->GetOwner())
		{
			if (Actor->GetContentBundleGuid().IsValid())
			{
				return FLinearColor::MakeRandomSeededColor(GetTypeHash(Actor->GetContentBundleGuid()));
			}
		}
		return FLinearColor::Gray;
	},
	[](){}, 
	LOCTEXT("ContentBundleColor_ToolTip", "Colorize actor using a random color per content bundle, the rest is Gray.") );
#endif
}

void UContentBundleEditingSubmodule::DoDenitialize()
{
	if (GEditor)
	{
		UActorEditorContextSubsystem::Get()->UnregisterClient(this);
		GEditor->OnLevelActorAdded().RemoveAll(this);
	}

	EditingContentBundleGuid.Invalidate();
	if (GEngine)
	{
		UContentBundleEngineSubsystem::Get()->SetEditingContentBundleGuid(EditingContentBundleGuid);
	}

	EditingContentBundlesStack.Empty();

#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	FActorPrimitiveColorHandler::Get().UnregisterPrimitiveColorHandler(NAME_ContentBundleColor);
#endif
}

void UContentBundleEditingSubmodule::PreEditUndo()
{
	Super::PreEditUndo();

	PreUndoRedoEditingContentBundleGuid = EditingContentBundleGuid;
}

void UContentBundleEditingSubmodule::PostEditUndo()
{
	Super::PostEditUndo();

	if (PreUndoRedoEditingContentBundleGuid != EditingContentBundleGuid)
	{
		if (TSharedPtr<FContentBundleEditor> OldEditingContentBundle = GetEditorContentBundle(PreUndoRedoEditingContentBundleGuid))
		{
			StopEditing(OldEditingContentBundle);
		}
		if (TSharedPtr<FContentBundleEditor> NewEditingContentBundle = GetEditorContentBundle(EditingContentBundleGuid))
		{
			StartEditing(NewEditingContentBundle);
		}

		UContentBundleEngineSubsystem::Get()->SetEditingContentBundleGuid(EditingContentBundleGuid);
	}

	PreUndoRedoEditingContentBundleGuid.Invalidate();
}

void UContentBundleEditingSubmodule::ApplyContext(AActor* InActor)
{
	if (GIsReinstancing || !InActor || (InActor->GetWorld() != GetWorld()) || InActor->HasAnyFlags(RF_Transient) || InActor->IsChildActor() || !EditingContentBundleGuid.IsValid() || (InActor->GetContentBundleGuid() == EditingContentBundleGuid))
	{
		return;
	}

	// Prefer override spawning External Data Layer over Content Bundles
	if (const UExternalDataLayerInstance* SpawningExternalDataLayerInstance = ULevel::GetOverrideSpawningLevelMountPointObject() ? UDataLayerEditorSubsystem::Get()->GetActorSpawningExternalDataLayerInstance(InActor) : nullptr)
	{
		return;
	}

	if (TSharedPtr<FContentBundleEditor> EditingContentBundle = GetEditorContentBundle(EditingContentBundleGuid))
	{
		EditingContentBundle->AddActor(InActor);
	}
}

void UContentBundleEditingSubmodule::OnExecuteActorEditorContextAction(UWorld* InWorld, const EActorEditorContextAction& InType, AActor* InActor /* = nullptr */)
{
	switch (InType)
	{
	case EActorEditorContextAction::ApplyContext:
		// For backward compatibility, Content Bundle will only apply context trough OnLevelActorAdded
		break;
	case EActorEditorContextAction::ResetContext:
		if (TSharedPtr<FContentBundleEditor> EditingContentBundle = GetEditorContentBundle(EditingContentBundleGuid))
		{
			DeactivateCurrentContentBundleEditing();
		}
		break;
	case EActorEditorContextAction::PushContext:
	case EActorEditorContextAction::PushDuplicateContext:
		PushContentBundleEditing(InType == EActorEditorContextAction::PushDuplicateContext);
		break;
	case EActorEditorContextAction::PopContext:
		PopContentBundleEditing();
		break;
	case EActorEditorContextAction::InitializeContextFromActor:
		{
			if (InActor->GetContentBundleGuid().IsValid())
			{
				if (TSharedPtr<FContentBundleEditor> EditingContentBundle = GetEditorContentBundle(InActor->GetContentBundleGuid()))
				{
					ActivateContentBundleEditing(EditingContentBundle);
				}
			}
		}
		break;
	}
}

void UContentBundleEditingSubmodule::CaptureActorEditorContextState(UWorld* InWorld, UActorEditorContextStateCollection* InStateCollection) const
{
	if (EditingContentBundleGuid.IsValid())
	{
		UActorEditorContextContentBundleState* State = NewObject<UActorEditorContextContentBundleState>(InStateCollection);
		State->ContentBundleGuid = EditingContentBundleGuid;
		InStateCollection->AddState(State);
	}
}

void UContentBundleEditingSubmodule::RestoreActorEditorContextState(UWorld* InWorld, const UActorEditorContextStateCollection* InStateCollection)
{
	TSharedPtr<FContentBundleEditor> ContentBundleToActivate;

	if (const UActorEditorContextContentBundleState* State = InStateCollection->GetState<UActorEditorContextContentBundleState>())
	{
		ContentBundleToActivate = GetEditorContentBundle(State->ContentBundleGuid);
	}

	if (ContentBundleToActivate.IsValid())
	{
		ActivateContentBundleEditing(ContentBundleToActivate);
	}
	else
	{
		DeactivateCurrentContentBundleEditing();
	}
}

bool UContentBundleEditingSubmodule::GetActorEditorContextDisplayInfo(UWorld* InWorld, FActorEditorContextClientDisplayInfo& OutDiplayInfo) const
{
	if (EditingContentBundleGuid.IsValid())
	{
		constexpr TCHAR PLACEHOLDER_ContentBundle_Brush[] = TEXT("DataLayer.Editor"); // todo_ow: create placeholder brush for content bundle
		OutDiplayInfo.Title = TEXT("Content Bundle");
		OutDiplayInfo.Brush = FAppStyle::GetBrush(PLACEHOLDER_ContentBundle_Brush);
		return true;
	}

	return false;
}

TSharedRef<SWidget> UContentBundleEditingSubmodule::GetActorEditorContextWidget(UWorld* World) const
{
	constexpr TCHAR PLACEHOLDER_ContentBundle_ColorIcon[] = TEXT("DataLayer.ColorIcon"); // todo_ow: create placeholder color icon for content bundle

	TSharedRef<SVerticalBox> OutWidget = SNew(SVerticalBox);

	if (EditingContentBundleGuid.IsValid())
	{
		OutWidget->AddSlot().AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.0f, 1.0f, 1.0f, 1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.ColorAndOpacity_Lambda([this]()
				{
					if (TSharedPtr<FContentBundleEditor> ContentBundleEditor = GetEditorContentBundle(EditingContentBundleGuid))
					{
						return ContentBundleEditor->GetDebugColor();
					}
					return FColor::Black;
				})
				.Image(FAppStyle::Get().GetBrush(PLACEHOLDER_ContentBundle_ColorIcon))
				.DesiredSizeOverride(FVector2D(8, 8))
			]
		+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 1.0f, 1.0f, 1.0f)
			[
				SNew(STextBlock)
				.Text_Lambda([this]()
				{
					if (TSharedPtr<FContentBundleEditor> ContentBundleEditor = GetEditorContentBundle(EditingContentBundleGuid))
					{
						if (!ContentBundleEditor->GetDisplayName().IsEmpty())
						{
							return FText::FromString(ContentBundleEditor->GetDisplayName());
						}
					}
					return FText::FromString(TEXT("Unset Display Name"));
				})
			]
		];
	}

	return OutWidget;
}

bool UContentBundleEditingSubmodule::ActivateContentBundleEditing(TSharedPtr<FContentBundleEditor>& ContentBundleEditor)
{
	Modify();

	EContentBundleStatus ContentBundleStatus = ContentBundleEditor->GetStatus();
	bool bCanActivateEditing = ContentBundleStatus == EContentBundleStatus::ContentInjected || ContentBundleStatus == EContentBundleStatus::ReadyToInject;
	if (bCanActivateEditing && !ContentBundleEditor->IsBeingEdited())
	{
		if (IsEditingContentBundle())
		{
			check(!IsEditingContentBundle(ContentBundleEditor));
			DeactivateCurrentContentBundleEditing();
		}
		EditingContentBundleGuid = ContentBundleEditor->GetDescriptor()->GetGuid();
		UContentBundleEngineSubsystem::Get()->SetEditingContentBundleGuid(EditingContentBundleGuid);
		StartEditing(ContentBundleEditor);
		return true;
	}
	return false;
}

bool UContentBundleEditingSubmodule::DeactivateCurrentContentBundleEditing()
{
	if (TSharedPtr<FContentBundleEditor> ContentBundleToDeactivate = GetEditorContentBundle(EditingContentBundleGuid))
	{
		return DeactivateContentBundleEditing(ContentBundleToDeactivate);
	}
	return false;
}

bool UContentBundleEditingSubmodule::DeactivateContentBundleEditing(TSharedPtr<FContentBundleEditor>& ContentBundleEditor)
{
	Modify();

	if (IsEditingContentBundle(ContentBundleEditor))
	{
		check(EditingContentBundleGuid == ContentBundleEditor->GetDescriptor()->GetGuid());
		EditingContentBundleGuid.Invalidate();
		UContentBundleEngineSubsystem::Get()->SetEditingContentBundleGuid(EditingContentBundleGuid);
		StopEditing(ContentBundleEditor);
		return true;
	}

	return false;
}

void UContentBundleEditingSubmodule::StartEditing(TSharedPtr<FContentBundleEditor>& ContentBundleEditor)
{
	check(ContentBundleEditor.IsValid());
	check(!ContentBundleEditor->IsBeingEdited());
	ContentBundleEditor->StartEditing();
	ActorEditorContextClientChanged.Broadcast(this);
	GetSubsystem()->NotifyContentBundleChanged(ContentBundleEditor.Get());
	UE_LOGF(LogContentBundle, Log, "[CB: %ls] Content Bundle is being edited", *ContentBundleEditor->GetDescriptor()->GetDisplayName());
}

void UContentBundleEditingSubmodule::StopEditing(TSharedPtr<FContentBundleEditor>& ContentBundleEditor)
{
	check(ContentBundleEditor.IsValid());
	check(ContentBundleEditor->IsBeingEdited());
	ContentBundleEditor->StopEditing();
	ActorEditorContextClientChanged.Broadcast(this);
	GetSubsystem()->NotifyContentBundleChanged(ContentBundleEditor.Get());
	UE_LOGF(LogContentBundle, Log, "[CB: %ls] Content Bundle is no longer being edited", *ContentBundleEditor->GetDescriptor()->GetDisplayName());
}

void UContentBundleEditingSubmodule::PushContentBundleEditing(bool bDuplicateContext)
{
	Modify();

	EditingContentBundlesStack.Add(EditingContentBundleGuid);
	if (!bDuplicateContext)
	{
		DeactivateCurrentContentBundleEditing();
	}
}

void UContentBundleEditingSubmodule::PopContentBundleEditing()
{
	Modify();

	DeactivateCurrentContentBundleEditing();
	const FGuid ContentBundleGuidToActivate = EditingContentBundlesStack.Pop();
	if (TSharedPtr<FContentBundleEditor> ContentBundleToActivate = GetEditorContentBundle(ContentBundleGuidToActivate))
	{
		ActivateContentBundleEditing(ContentBundleToActivate);
	}
}

bool UContentBundleEditingSubmodule::IsEditingContentBundle() const
{
	return IsEditingContentBundle(EditingContentBundleGuid);
}

bool UContentBundleEditingSubmodule::IsEditingContentBundle(const TSharedPtr<FContentBundleEditor>& ContentBundleEditor) const
{
	return ContentBundleEditor.IsValid() && IsEditingContentBundle(ContentBundleEditor->GetDescriptor()->GetGuid());
}

bool UContentBundleEditingSubmodule::IsEditingContentBundle(const FGuid& ContentBundleGuid) const
{
	return ContentBundleGuid.IsValid() && (ContentBundleGuid == EditingContentBundleGuid) && GetEditorContentBundle(ContentBundleGuid);
}

TSharedPtr<FContentBundleEditor> UContentBundleEditingSubmodule::GetEditorContentBundle(const FGuid& ContentBundleGuid) const
{
	return GetSubsystem()->GetEditorContentBundle(ContentBundleGuid);
}

void UContentBundleEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Collection.InitializeDependency<UActorEditorContextSubsystem>();

	ContentBundleEditingSubModule = NewObject<UContentBundleEditingSubmodule>(this, NAME_None, RF_Transactional);
	ContentBundleEditingSubModule->Initialize();

	LevelExternalActorsPathsProviderDelegateHandle = ULevel::RegisterLevelExternalActorsPathsProvider(ULevel::FLevelExternalActorsPathsProviderDelegate::CreateUObject(this, &UContentBundleEditorSubsystem::OnGetLevelExternalActorsPaths));

	IContentBundleEditorSubsystemInterface::SetInstance(this);
}

void UContentBundleEditorSubsystem::Deinitialize()
{
	IContentBundleEditorSubsystemInterface::SetInstance(nullptr);

	ULevel::UnregisterLevelExternalActorsPathsProvider(LevelExternalActorsPathsProviderDelegateHandle);

	ContentBundleEditingSubModule->Deinitialize();
	ContentBundleEditingSubModule = nullptr;
}

UWorld* UContentBundleEditorSubsystem::GetWorld() const
{
	return GEditor->GetEditorWorldContext().World();
}

void UContentBundleEditorSubsystem::NotifyContentBundleInjectedContent(const FContentBundleEditor* ContentBundle)
{
	NotifyContentBundleChanged(ContentBundle);
}

void UContentBundleEditorSubsystem::NotifyContentBundleRemovedContent(const FContentBundleEditor* ContentBundle)
{
	if (ContentBundle->IsBeingEdited())
	{
		GetEditingSubmodule()->DeactivateCurrentContentBundleEditing();
	}

	OnContentBundleRemovedContent().Broadcast(ContentBundle);
	NotifyContentBundleChanged(ContentBundle);
}

TSharedPtr<FContentBundleEditor> UContentBundleEditorSubsystem::GetEditorContentBundleForActor(const AActor* Actor)
{
	for (TSharedPtr<FContentBundleEditor>& ContentBundle : GetEditorContentBundles())
	{
		if (ContentBundle->ContainsActor(Actor))
		{
			return ContentBundle;
		}
	}

	return nullptr;
}

TArray<TSharedPtr<FContentBundleEditor>> UContentBundleEditorSubsystem::GetEditorContentBundles()
{
	TArray<TSharedPtr<FContentBundleEditor>> EditorContentBundles;
	if (UContentBundleManager* ContentBundleManager = GetWorld()->ContentBundleManager)
	{
		ContentBundleManager->GetEditorContentBundle(EditorContentBundles);
	}

	return EditorContentBundles;
}

TSharedPtr<FContentBundleEditor> UContentBundleEditorSubsystem::GetEditorContentBundle(const UContentBundleDescriptor* ContentBundleDescriptor) const
{
	if (UContentBundleManager* ContentBundleManager = GetWorld()->ContentBundleManager)
	{
		return ContentBundleManager->GetEditorContentBundle(ContentBundleDescriptor, GetWorld());
	}

	return nullptr;
}

TSharedPtr<FContentBundleEditor> UContentBundleEditorSubsystem::GetEditorContentBundle(const FGuid& ContentBundleGuid) const
{
	if (ContentBundleGuid.IsValid())
	{
		if (UContentBundleManager* ContentBundleManager = GetWorld()->ContentBundleManager)
		{
			return ContentBundleManager->GetEditorContentBundle(ContentBundleGuid);
		}
	}

	return nullptr;
}

bool UContentBundleEditorSubsystem::IsEditingContentBundle() const
{
	return ContentBundleEditingSubModule->IsEditingContentBundle();
}

bool UContentBundleEditorSubsystem::IsEditingContentBundle(const FGuid& ContentBundleGuid) const
{
	return ContentBundleEditingSubModule->IsEditingContentBundle(ContentBundleGuid);
}

void UContentBundleEditorSubsystem::SelectActors(FContentBundleEditor& EditorContentBundle)
{
	SelectActorsInternal(EditorContentBundle, true);
}

void UContentBundleEditorSubsystem::DeselectActors(FContentBundleEditor& EditorContentBundle)
{
	SelectActorsInternal(EditorContentBundle, false);
}

void UContentBundleEditorSubsystem::ReferenceAllActors(FContentBundleEditor& EditorContentBundle)
{
	EditorContentBundle.ReferenceAllActors();
}

void UContentBundleEditorSubsystem::UnreferenceAllActors(FContentBundleEditor& EditorContentBundle)
{
	EditorContentBundle.UnreferenceAllActors();
}

bool UContentBundleEditorSubsystem::ActivateContentBundleEditing(TSharedPtr<FContentBundleEditor>& ContentBundleEditor)  const
{
	return ContentBundleEditingSubModule->ActivateContentBundleEditing(ContentBundleEditor);
}

bool UContentBundleEditorSubsystem::DeactivateContentBundleEditing(TSharedPtr<FContentBundleEditor>& ContentBundleEditor) const
{
	return ContentBundleEditingSubModule->DeactivateContentBundleEditing(ContentBundleEditor);
}

bool UContentBundleEditorSubsystem::DeactivateCurrentContentBundleEditing() const
{
	return ContentBundleEditingSubModule->DeactivateCurrentContentBundleEditing();
}

bool UContentBundleEditorSubsystem::IsContentBundleEditingActivated(TSharedPtr<FContentBundleEditor>& ContentBundleEditor) const
{
	return ContentBundleEditor.IsValid() && IsEditingContentBundle(ContentBundleEditor->GetDescriptor()->GetGuid());
}

void UContentBundleEditorSubsystem::PushContentBundleEditing(bool bDuplicateContext)
{
	ContentBundleEditingSubModule->PushContentBundleEditing(bDuplicateContext);
}

void UContentBundleEditorSubsystem::PopContentBundleEditing()
{
	ContentBundleEditingSubModule->PopContentBundleEditing();
}

void UContentBundleEditorSubsystem::SelectActorsInternal(FContentBundleEditor& EditorContentBundle, bool bSelect)
{
	GEditor->GetSelectedActors()->BeginBatchSelectOperation();
	{
		TArray<AActor*> ContentBundleActors;
		EditorContentBundle.GetActors(ContentBundleActors);

		for (AActor* Actor : ContentBundleActors)
		{
			bool bNotifyForActor = false;
			GEditor->GetSelectedActors()->Modify();
			GEditor->SelectActor(Actor, bSelect, true, false);
		}
	}
	GEditor->GetSelectedActors()->EndBatchSelectOperation();

	GEditor->NoteSelectionChange();
}

void UContentBundleEditorSubsystem::OnGetLevelExternalActorsPaths(const FString& InLevelPackageName, const FString& InPackageShortName, TArray<FString>& OutExternalActorsPaths)
{
	ContentBundlePaths::ForEachContentBundleLevelPackagePath(InLevelPackageName, [&OutExternalActorsPaths](const FString& Path)
	{
		const FString ExternalActorPath = ULevel::GetExternalActorsPath(Path);
		OutExternalActorsPaths.Add(ExternalActorPath);
	});
}

#undef LOCTEXT_NAMESPACE
