// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/ContentBundle/Outliner/ContentBundleActorCountColumn.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/STreeView.h"
#include "ScopedTransaction.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "WorldPartition/ContentBundle/Outliner/ContentBundleTreeItem.h"
#include "WorldPartition/ContentBundle/ContentBundleEditor.h"

#define LOCTEXT_NAMESPACE "ContentBundle"

namespace ContentBundleOutlinerPrivate
{
	FName ContentBundleOutlinerBundleActorCount("Content Bundle Actor Count");
}

FName FContentBundleOutlinerActorCountColumn::GetID()
{
	return ContentBundleOutlinerPrivate::ContentBundleOutlinerBundleActorCount;
}

SHeaderRow::FColumn::FArguments FContentBundleOutlinerActorCountColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultTooltip(LOCTEXT("ContentBundleOutlinerActorCountTooltip", "Actor Count (Unsaved Actor Count)"))
		[
			SNew(STextBlock)
			.Margin(FMargin(0.0f))
			.Text(LOCTEXT("ContentBundleOutlinerActorCount", "Actor Count"))
		];
}

const TSharedRef<SWidget> FContentBundleOutlinerActorCountColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	if (FContentBundleTreeItem* ContentBundleTreeItem = TreeItem->CastTo<FContentBundleTreeItem>())
	{
		TSharedPtr<FContentBundleEditor> ContentBundleEditor = ContentBundleTreeItem->GetContentBundleEditorPin();
		if (ContentBundleEditor != nullptr)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(0, 0, 0, 0)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text_Lambda([ContentBundleTreeItem]
						{
							TSharedPtr<FContentBundleEditor> ContentBundleEditorPin = ContentBundleTreeItem->GetContentBundleEditorPin();
							if (ContentBundleEditorPin != nullptr)
							{
								FString ActorCountString = FString::Printf(TEXT("%u(%u)"), ContentBundleEditorPin->GetActorCount(), ContentBundleEditorPin->GetUnsavedActorAcount());
								return FText::FromString(ActorCountString);
							}

							return FText::AsNumber(0);
						})
					.ColorAndOpacity(MakeAttributeLambda([ContentBundleTreeItem] { return ContentBundleTreeItem->GetItemColor(); }))
				];
		}
	}
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE