// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/ContentBundle/Outliner/ContentBundleClientColumn.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/STreeView.h"
#include "ScopedTransaction.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "WorldPartition/ContentBundle/Outliner/ContentBundleTreeItem.h"
#include "WorldPartition/ContentBundle/ContentBundleEditor.h"
#include "WorldPartition/ContentBundle/ContentBundle.h"
#include "WorldPartition/ContentBundle/ContentBundleClient.h"
#include "SortHelper.h"

#define LOCTEXT_NAMESPACE "ContentBundle"

namespace ContentBundleOutlinerPrivate
{
	FName ContentBundleOutlinerClient("Content Bundle Client");
}

FName FContentBundleOutlinerClientColumn::GetID()
{
	return ContentBundleOutlinerPrivate::ContentBundleOutlinerClient;
}

SHeaderRow::FColumn::FArguments FContentBundleOutlinerClientColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultTooltip(FText::FromName(GetColumnID()))
		[
			SNew(STextBlock)
			.Margin(FMargin(0.0f))
			.Text(LOCTEXT("ContentBundleOutlinerClient", "Client"))
		];
}

const TSharedRef<SWidget> FContentBundleOutlinerClientColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	if (FContentBundleTreeItem* ContentBundleTreeItem = TreeItem->CastTo<FContentBundleTreeItem>())
	{
		TWeakPtr<FContentBundleEditor> ContentBundleEditor = ContentBundleTreeItem->GetContentBundleEditor();
		if (ContentBundleEditor != nullptr)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(0, 0, 0, 0)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text_Lambda([this, ContentBundleEditor] { return GetClientDisplayName(ContentBundleEditor); })
					.ColorAndOpacity(MakeAttributeLambda([ContentBundleTreeItem] { return ContentBundleTreeItem->GetItemColor(); }))
				];
		}
	}
	return SNullWidget::NullWidget;
}

void FContentBundleOutlinerClientColumn::SortItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems, const EColumnSortMode::Type SortMode) const
{
	typedef FSceneOutlinerSortHelper<int32, SceneOutliner::FNumericStringWrapper> FSort;

	auto SortByClientName = [this](const ISceneOutlinerTreeItem& Item)
	{
		if (const FContentBundleTreeItem* ContentBundleTreeItem = Item.CastTo<FContentBundleTreeItem>())
		{
			TSharedPtr<FContentBundleEditor> ContentBundleEditorPin = ContentBundleTreeItem->GetContentBundleEditorPin();
			if (ContentBundleEditorPin != nullptr)
			{
				TSharedPtr<FContentBundleClient> Client = ContentBundleEditorPin->GetClient().Pin();
				if (Client != nullptr)
				{
					FString Temp = Client->GetDisplayName();
					return SceneOutliner::FNumericStringWrapper(MoveTemp(Temp));
				}
			}
		}

		return SceneOutliner::FNumericStringWrapper();
	};

	FSort()
		.Primary([this](const ISceneOutlinerTreeItem& Item) { return WeakSceneOutliner.Pin()->GetTypeSortPriority(Item); }, SortMode)
		.Secondary(SortByClientName, SortMode)
		.Sort(OutItems);
}

FText FContentBundleOutlinerClientColumn::GetClientDisplayName(const TWeakPtr<FContentBundleEditor>& ContentBundleEditor) const
{
	TSharedPtr<FContentBundleEditor> ContentBundleEditorPin = ContentBundleEditor.Pin();
	if (ContentBundleEditorPin != nullptr)
	{
		TSharedPtr<FContentBundleClient> ContentBundleClient = ContentBundleEditorPin->GetClient().Pin();
		if (ContentBundleClient != nullptr)
		{
			return FText::FromString(ContentBundleClient->GetDisplayName());
		}
	}
	

	return LOCTEXT("Unknow", "Unknown");
}

#undef LOCTEXT_NAMESPACE