// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/ContentBundle/Outliner/ContentBundleEditingColumn.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/STreeView.h"
#include "ScopedTransaction.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "WorldPartition/ContentBundle/Outliner/ContentBundleTreeItem.h"
#include "WorldPartition/ContentBundle/ContentBundleEditor.h"

#define LOCTEXT_NAMESPACE "ContentBundle"

namespace ContentBundleOutlinerPrivate
{
	FName ContentBundleOutlinerEditing("Editing");
}

FName FContentBundleOutlinerEditingColumn::GetID()
{
	return ContentBundleOutlinerPrivate::ContentBundleOutlinerEditing;
}

SHeaderRow::FColumn::FArguments FContentBundleOutlinerEditingColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultTooltip(FText::FromName(GetColumnID()))
		[
			SNew(STextBlock)
			.Margin(FMargin(0.0f))
			.Text(LOCTEXT("ContentBundleOutlinerEditing", "Editing"))

		];
}

const TSharedRef<SWidget> FContentBundleOutlinerEditingColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	if (FContentBundleTreeItem* ContentBundleTreeItem = TreeItem->CastTo<FContentBundleTreeItem>())
	{
		TWeakPtr<FContentBundleEditor> ContentBundleEditor = ContentBundleTreeItem->GetContentBundleEditor();
		if (ContentBundleEditor != nullptr)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(0, 0, 0, 0)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text_Lambda([ContentBundleTreeItem] 
						{ 
							TSharedPtr<FContentBundleEditor> ContentBundleEditorPin = ContentBundleTreeItem->GetContentBundleEditorPin();
							if (ContentBundleEditorPin != nullptr && ContentBundleEditorPin->IsBeingEdited())
							{
								return LOCTEXT("ContentBundleOutlinerEditingActive", "Active");
							}
							
							return FText();
						})
					.ColorAndOpacity(MakeAttributeLambda([ContentBundleTreeItem] { return ContentBundleTreeItem->GetItemColor(); }))
				];
		}
	}
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE