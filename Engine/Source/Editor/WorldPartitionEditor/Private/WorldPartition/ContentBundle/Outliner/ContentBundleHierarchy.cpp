// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/ContentBundle/Outliner/ContentBundleHierarchy.h"

#include "WorldPartition/ContentBundle/Outliner/ContentBundleMode.h"
#include "WorldPartition/ContentBundle/ContentBundleEditorSubsystem.h"
#include "WorldPartition/ContentBundle/ContentBundleEditor.h"
#include "WorldPartition/ContentBundle/Outliner/ContentBundleTreeItem.h"
#include "WorldTreeItem.h"

FContentBundleHiearchy::FContentBundleHiearchy(FContentBundleMode* Mode)
	:ISceneOutlinerHierarchy(Mode)
{
	UContentBundleEditorSubsystem::Get()->OnContentBundleChanged().AddRaw(this, &FContentBundleHiearchy::OnContentBundleChanged);
	UContentBundleEditorSubsystem::Get()->OnContentBundleAdded().AddRaw(this, &FContentBundleHiearchy::OnContentBundleAdded);
	UContentBundleEditorSubsystem::Get()->OnContentBundleRemoved().AddRaw(this, &FContentBundleHiearchy::OnContentBundleRemoved);
}

FContentBundleHiearchy::~FContentBundleHiearchy()
{
	if (UContentBundleEditorSubsystem* ContentBundleEditorSubsystem = UContentBundleEditorSubsystem::Get())
	{
		ContentBundleEditorSubsystem->OnContentBundleChanged().RemoveAll(this);
		ContentBundleEditorSubsystem->OnContentBundleAdded().RemoveAll(this);
		ContentBundleEditorSubsystem->OnContentBundleRemoved().RemoveAll(this);
	}
}

TUniquePtr<FContentBundleHiearchy> FContentBundleHiearchy::Create(FContentBundleMode* Mode)
{
	return TUniquePtr<FContentBundleHiearchy>(new FContentBundleHiearchy(Mode));
}

void FContentBundleHiearchy::CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	UContentBundleEditorSubsystem* ContentBundleEditorSubsystem = UContentBundleEditorSubsystem::Get();
	if (ContentBundleEditorSubsystem != nullptr)
	{
		for (TSharedPtr<FContentBundleEditor>& ContentBundleEditor : ContentBundleEditorSubsystem->GetEditorContentBundles())
		{
			if (FSceneOutlinerTreeItemPtr ContentBundleTreeItem = Mode->CreateItemFor<FContentBundleTreeItem>(FContentBundleTreeItem::FInitializationValues(ContentBundleEditor, *StaticCast<FContentBundleMode*>(Mode)), true))
			{
				OutItems.Add(ContentBundleTreeItem);
			}
		}
	}	
}

void FContentBundleHiearchy::CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutChildren) const
{

}

FSceneOutlinerTreeItemPtr FContentBundleHiearchy::FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate /* = false */)
{
	if (const FContentBundleTreeItem* ContentBundleTreeItem = Item.CastTo<FContentBundleTreeItem>())
	{
		TSharedPtr<FContentBundleEditor> ContentBundleEditorPin = ContentBundleTreeItem->GetContentBundleEditorPin();
		if (ContentBundleEditorPin != nullptr)
		{
			if (UWorld* World = ContentBundleEditorPin->GetInjectedWorld())
			{
				if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(World))
				{
					return *ParentItem;
				}
				else if (bCreate)
				{
					return Mode->CreateItemFor<FWorldTreeItem>(World, true);
				}
			}
		}
	}

	return nullptr;
}

void FContentBundleHiearchy::OnContentBundleChanged(const FContentBundleEditor* ContentBundle)
{
	FSceneOutlinerHierarchyChangedData EventData;
	EventData.Type = FSceneOutlinerHierarchyChangedData::FullRefresh;
	EventData.ItemIDs.Emplace(ContentBundle->GetTreeItemID());
	HierarchyChangedEvent.Broadcast(EventData);
}

void FContentBundleHiearchy::OnContentBundleAdded(const FContentBundleEditor* ContentBundle)
{
	FSceneOutlinerHierarchyChangedData EventData;
	EventData.Type = FSceneOutlinerHierarchyChangedData::FullRefresh;
	HierarchyChangedEvent.Broadcast(EventData);
}

void FContentBundleHiearchy::OnContentBundleRemoved(const FContentBundleEditor* ContentBundle)
{
	FSceneOutlinerHierarchyChangedData EventData;
	EventData.Type = FSceneOutlinerHierarchyChangedData::FullRefresh;
	HierarchyChangedEvent.Broadcast(EventData);
}