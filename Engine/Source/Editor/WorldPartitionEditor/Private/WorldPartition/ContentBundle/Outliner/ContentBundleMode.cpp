// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/ContentBundle/Outliner/ContentBundleMode.h"
#include "WorldPartition/ContentBundle/Outliner/ContentBundleHierarchy.h"
#include "WorldPartition/ContentBundle/Outliner/ContentBundleTreeItem.h"
#include "WorldPartition/ContentBundle/ContentBundleEditorSubsystem.h"
#include "WorldPartition/ContentBundle/ContentBundleEditor.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "DataLayer/DataLayerEditorSubsystem.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "SceneOutlinerMenuContext.h"
#include "Algo/Transform.h"
#include "Engine/DataAsset.h"
#include "ScopedTransaction.h"
#include "ToolMenus.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "ContentBundle"

namespace ContentBundleOutlinerPrivate
{
	const FName DefaultContextBaseMenuName("ContentBundleOutliner.DefaultContextMenuBase");
	const FName DefaultContextMenuName("ContentBundleOutliner.DefaultContextMenu");
}

FContentBundleMode::FContentBundleMode(const FContentBundleModeCreationParams& Params)
	:ISceneOutlinerMode(Params.SceneOutliner)
{
	Commands = MakeShareable(new FUICommandList());
	Commands->MapAction(FGlobalEditorCommonCommands::Get().FindInContentBrowser, FUIAction(
		FExecuteAction::CreateRaw(this, &FContentBundleMode::FindInContentBrowser),
		FCanExecuteAction::CreateRaw(this, &FContentBundleMode::CanFindInContentBrowser)
	));
}

FContentBundleMode::~FContentBundleMode()
{
	UnregisterContextMenu();
}

void FContentBundleMode::Rebuild()
{
	Hierarchy = CreateHierarchy();
}

TSharedPtr<SWidget> FContentBundleMode::CreateContextMenu()
{
	RegisterContextMenu();

	FSceneOutlinerItemSelection ItemSelection(SceneOutliner->GetSelection());

	USceneOutlinerMenuContext* ContextObject = NewObject<USceneOutlinerMenuContext>();
	ContextObject->SceneOutliner = StaticCastSharedRef<SSceneOutliner>(SceneOutliner->AsShared());
	ContextObject->bShowParentTree = SceneOutliner->GetSharedData().bShowParentTree;
	ContextObject->NumSelectedItems = ItemSelection.Num();
	FToolMenuContext Context(ContextObject);

	FName MenuName = ContentBundleOutlinerPrivate::DefaultContextMenuName;
	SceneOutliner->GetSharedData().ModifyContextMenu.ExecuteIfBound(MenuName, Context);

	// Build up the menu for a selection
	UToolMenus* ToolMenus = UToolMenus::Get();
	UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);
	for (const FToolMenuSection& Section : Menu->Sections)
	{
		if (Section.Blocks.Num() > 0)
		{
			return ToolMenus->GenerateWidget(Menu);
		}
	}

	return nullptr;
}

TUniquePtr<ISceneOutlinerHierarchy> FContentBundleMode::CreateHierarchy()
{
	TUniquePtr<FContentBundleHiearchy> ContentBundleHierarchy = FContentBundleHiearchy::Create(this);
	return ContentBundleHierarchy;
}

void FContentBundleMode::OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item)
{
	if (FContentBundleTreeItem* ContentBundleTreeItem = Item->CastTo<FContentBundleTreeItem>())
	{
		ToggleContentBundleActivation(ContentBundleTreeItem->GetContentBundleEditor());
	}
}

FReply FContentBundleMode::OnKeyDown(const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::F5)
	{
		SceneOutliner->FullRefresh();
		return FReply::Handled();
	}

	if (Commands->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

UWorld* FContentBundleMode::GetEditingWorld() const
{
	if (UContentBundleEditorSubsystem* ContentBundleEditorSubsystem = UContentBundleEditorSubsystem::Get())
	{
		if (UWorld* SubsystemWorld = ContentBundleEditorSubsystem->GetWorld())
		{
			ULevel* CurrentLevel = SubsystemWorld->GetCurrentLevel();
			if (CurrentLevel != nullptr)
			{
				return CurrentLevel->GetTypedOuter<UWorld>();
			}
		}
	}

	return nullptr;
}

void FContentBundleMode::FindInContentBrowser()
{
	if (SceneOutliner)
	{
		TArray<UObject*> Objects;
		SceneOutliner->GetSelection().ForEachItem<FContentBundleTreeItem>([&Objects](const FContentBundleTreeItem& Item)
		{
			if (TSharedPtr<FContentBundleEditor> ContentBundleEditor = Item.GetContentBundleEditorPin())
			{
				if (UDataAsset* DataAsset = ContentBundleEditor->GetDescriptor()->GetTypedOuter<UDataAsset>())
				{
					Objects.Add(DataAsset);
				}
			}
		});
		if (!Objects.IsEmpty())
		{
			GEditor->SyncBrowserToObjects(Objects);
		}
	}
}

bool FContentBundleMode::CanFindInContentBrowser() const
{
	return SceneOutliner && SceneOutliner->GetSelection().Has<FContentBundleTreeItem>();
}

void FContentBundleMode::RegisterContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();

	if (!ToolMenus->IsMenuRegistered(ContentBundleOutlinerPrivate::DefaultContextBaseMenuName))
	{
		UToolMenu* Menu = ToolMenus->RegisterMenu(ContentBundleOutlinerPrivate::DefaultContextBaseMenuName);

		Menu->AddDynamicSection("ContentBundleDynamicSection", FNewToolMenuDelegate::CreateLambda([this](UToolMenu* InMenu)
			{
				USceneOutlinerMenuContext* Context = InMenu->FindContext<USceneOutlinerMenuContext>();
				if (!Context || !Context->SceneOutliner.IsValid())
				{
					return;
				}

				SSceneOutliner* SceneOutliner = Context->SceneOutliner.Pin().Get();

				{
					FToolMenuSection& Section = InMenu->AddSection("ContentBundleActorEditorContext", LOCTEXT("ContentBundleActorEditorContext", "Actor Editor Context"));

					Section.AddMenuEntry("MakeCurrentContentBundle", LOCTEXT("MakeCurrentContentBundle", "Make Current Content Bundle"), FText(), FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([this, SceneOutliner]()
							{
								TSharedPtr<FContentBundleEditor> SelectedContentBundleEditor = GetSelectedContentBundlePin(SceneOutliner);
								if (SelectedContentBundleEditor && !SelectedContentBundleEditor->IsBeingEdited())
								{
									ToggleContentBundleActivation(SelectedContentBundleEditor);
								}
							}),
							FCanExecuteAction::CreateLambda([this, SceneOutliner]
							{
								TSharedPtr<FContentBundleEditor> SelectedContentBundleEditor = GetSelectedContentBundlePin(SceneOutliner);
								if (SelectedContentBundleEditor && !SelectedContentBundleEditor->IsBeingEdited())
								{
									return !UDataLayerEditorSubsystem::Get()->GetActorEditorContextCurrentExternalDataLayer();
								}
								return false;
							})
						));

					Section.AddMenuEntry("ClearCurrentContentBundle", LOCTEXT("ClearCurrentContentBundle", "Clear Current Content Bundle"), FText(), FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([this, SceneOutliner]()
								{
									UContentBundleEditorSubsystem* ContentBundleEditorSubsystem = UContentBundleEditorSubsystem::Get();
									if (ContentBundleEditorSubsystem && ContentBundleEditorSubsystem->IsEditingContentBundle())
									{
										if (UContentBundleEditingSubmodule* ContentBundleEditingSubModule = ContentBundleEditorSubsystem->GetEditingSubmodule())
										{
											const FScopedTransaction Transaction(LOCTEXT("ClearCurrentContentBundle", "Clear Current Content Bundle"));
											ContentBundleEditingSubModule->DeactivateCurrentContentBundleEditing();
										}
									}
								}),
							FCanExecuteAction::CreateLambda([this, SceneOutliner]
								{
									UContentBundleEditorSubsystem* ContentBundleEditorSubsystem = UContentBundleEditorSubsystem::Get();
									return ContentBundleEditorSubsystem && ContentBundleEditorSubsystem->IsEditingContentBundle();
								})
						));
				}
				
				{
					FToolMenuSection& Section = InMenu->AddSection("ContentBundleSection", LOCTEXT("ContentBundle", "Content Bundle"));

					Section.AddMenuEntry("InjectBaseContentMenuEntry", LOCTEXT("InjectBaseContent", "Inject Base Content"), FText(), FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([this, SceneOutliner]
								{
									if (TSharedPtr<FContentBundleEditor> SelectedContentBundleEditor = GetSelectedContentBundlePin(SceneOutliner))
									{
										SelectedContentBundleEditor->InjectBaseContent();
									}
								}),
							FCanExecuteAction::CreateLambda([this, SceneOutliner]
								{
									return false;
								})
							));
				}

				{
					FToolMenuSection& Section = InMenu->AddSection("AssetOptionsSection", LOCTEXT("AssetOptionsText", "Asset Options"));
					Section.AddMenuEntryWithCommandList(FGlobalEditorCommonCommands::Get().FindInContentBrowser, Commands);
				}

				{
					FToolMenuSection& Section = InMenu->AddSection("ContentBundleLoadingSection", LOCTEXT("Loading", "Loading"));

					Section.AddMenuEntry("LoadActorsEntry", LOCTEXT("LoadActors", "Load Actors"), FText(), FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([this, SceneOutliner]
							{
								TSharedPtr<FContentBundleEditor> SelectedContentBundleEditor = GetSelectedContentBundlePin(SceneOutliner);
								if (SelectedContentBundleEditor != nullptr)
								{
									UContentBundleEditorSubsystem::Get()->ReferenceAllActors(*SelectedContentBundleEditor);
								}
							}),
							FCanExecuteAction::CreateLambda([this]
							{
								return true;
							})
						));

					Section.AddMenuEntry("UnloadActorsEntry", LOCTEXT("UnloadActors", "Unload Actors"), FText(), FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([this, SceneOutliner]
							{
								TSharedPtr<FContentBundleEditor> SelectedContentBundleEditor = GetSelectedContentBundlePin(SceneOutliner);
								if (SelectedContentBundleEditor != nullptr)
								{
									UContentBundleEditorSubsystem::Get()->UnreferenceAllActors(*SelectedContentBundleEditor);
								}
							}),
							FCanExecuteAction::CreateLambda([this]
							{
								return true;
							})
						));
				}

				{
					FToolMenuSection& Section = InMenu->AddSection("ContentBundleSelectionSection", LOCTEXT("Selection", "Selection"));

					Section.AddMenuEntry("SelectActorsEntry", LOCTEXT("SelectActors", "Select Actors"), FText(), FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([this, SceneOutliner]
								{
									TSharedPtr<FContentBundleEditor> SelectedContentBundleEditor = GetSelectedContentBundlePin(SceneOutliner);
									if (SelectedContentBundleEditor != nullptr)
									{
										UContentBundleEditorSubsystem::Get()->SelectActors(*SelectedContentBundleEditor);
									}
								}),
							FCanExecuteAction::CreateLambda([this]
								{
									return true;
								})
							));

					Section.AddMenuEntry("DeselectActorsEntry", LOCTEXT("DeselectActors", "Deselect Actors"), FText(), FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([this, SceneOutliner]
								{
									TSharedPtr<FContentBundleEditor> SelectedContentBundleEditor = GetSelectedContentBundlePin(SceneOutliner);
									if (SelectedContentBundleEditor != nullptr)
									{
										UContentBundleEditorSubsystem::Get()->DeselectActors(*SelectedContentBundleEditor);
									}
								}),
							FCanExecuteAction::CreateLambda([this]
								{
									return true;
								})
							));
				}
			}));
	}

	if (!ToolMenus->IsMenuRegistered(ContentBundleOutlinerPrivate::DefaultContextMenuName))
	{
		ToolMenus->RegisterMenu(ContentBundleOutlinerPrivate::DefaultContextMenuName, ContentBundleOutlinerPrivate::DefaultContextBaseMenuName);
	}
}

void FContentBundleMode::UnregisterContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	ToolMenus->RemoveMenu(ContentBundleOutlinerPrivate::DefaultContextBaseMenuName);
	ToolMenus->RemoveMenu(ContentBundleOutlinerPrivate::DefaultContextMenuName);
}

void FContentBundleMode::ToggleContentBundleActivation(const TWeakPtr<FContentBundleEditor>& ContentBundle)
{
	if (UContentBundleEditorSubsystem* ContentBundleEditorSubsystem = UContentBundleEditorSubsystem::Get())
	{
		if (UContentBundleEditingSubmodule* ContentBundleEditingSubModule = ContentBundleEditorSubsystem->GetEditingSubmodule())
		{
			if (TSharedPtr<FContentBundleEditor> ContentBundlePin = ContentBundle.Pin())
			{
				if (ContentBundlePin->IsBeingEdited())
				{
					const FScopedTransaction Transaction(LOCTEXT("ClearCurrentContentBundle", "Clear Current Content Bundle"));
					ContentBundleEditingSubModule->DeactivateCurrentContentBundleEditing();
				}
				else
				{
					if (!UDataLayerEditorSubsystem::Get()->GetActorEditorContextCurrentExternalDataLayer())
					{
						const FScopedTransaction Transaction(LOCTEXT("MakeCurrentContentBundle", "Make Current Content Bundle"));
						ContentBundleEditingSubModule->ActivateContentBundleEditing(ContentBundlePin);
					}
				}
			}
		}
	}
}

TWeakPtr<FContentBundleEditor> FContentBundleMode::GetSelectedContentBundle(SSceneOutliner* InSceneOutliner) const
{
	FSceneOutlinerItemSelection ItemSelection(InSceneOutliner->GetSelection());
	
	TArray<FContentBundleTreeItem*> SelectedContentBundles;
	ItemSelection.Get<FContentBundleTreeItem>(SelectedContentBundles);
	
	TArray<TWeakPtr<FContentBundleEditor>> ValidSelectedContentBundle;
	Algo::TransformIf(SelectedContentBundles, ValidSelectedContentBundle, [](const auto Item) { return Item && Item->IsValid(); }, [](const auto Item) { return Item->GetContentBundleEditor(); });

	check(ValidSelectedContentBundle.Num() == 1 || ValidSelectedContentBundle.IsEmpty());
	return ValidSelectedContentBundle.IsEmpty() ? nullptr : ValidSelectedContentBundle[0];
}

TSharedPtr<FContentBundleEditor> FContentBundleMode::GetSelectedContentBundlePin(SSceneOutliner* InSceneOutliner) const
{
	return GetSelectedContentBundle(InSceneOutliner).Pin();
}

#undef LOCTEXT_NAMESPACE