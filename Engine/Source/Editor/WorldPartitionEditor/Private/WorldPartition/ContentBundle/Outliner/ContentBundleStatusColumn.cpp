// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/ContentBundle/Outliner/ContentBundleStatusColumn.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/STreeView.h"
#include "ScopedTransaction.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "WorldPartition/ContentBundle/Outliner/ContentBundleTreeItem.h"
#include "WorldPartition/ContentBundle/ContentBundleEditor.h"

#define LOCTEXT_NAMESPACE "ContentBundle"

namespace ContentBundleOutlinerPrivate
{
	FName ContentBundleOutlinerStatus("Content Bundle Status");
}

FName FContentBundleOutlinerStatusColumn::GetID()
{
	return ContentBundleOutlinerPrivate::ContentBundleOutlinerStatus;
}

SHeaderRow::FColumn::FArguments FContentBundleOutlinerStatusColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultTooltip(FText::FromName(GetColumnID()))
		[
			SNew(STextBlock)
			.Margin(FMargin(0.0f))
			.Text(LOCTEXT("ContentBundleOutlinerStatus", "Status"))
		];
}

const TSharedRef<SWidget> FContentBundleOutlinerStatusColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	if (FContentBundleTreeItem* ContentBundleTreeItem = TreeItem->CastTo<FContentBundleTreeItem>())
	{
		TWeakPtr<FContentBundleEditor> ContentBundleEditor = ContentBundleTreeItem->GetContentBundleEditor();
		if (ContentBundleEditor != nullptr)
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(0, 0, 0, 0)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text_Lambda([ContentBundleTreeItem]
						{ 
							TSharedPtr<FContentBundleEditor> ContentBundleEditorPin = ContentBundleTreeItem->GetContentBundleEditorPin();
							if (ContentBundleEditorPin != nullptr)
							{
								if (ContentBundleEditorPin->GetStatus() == EContentBundleStatus::FailedToInject)
								{
									return FText::Format(LOCTEXT("ConsultLogForErrors", "{0} (Consult Log For Errors)"), UEnum::GetDisplayValueAsText(ContentBundleEditorPin->GetStatus()));
								}
								
								return UEnum::GetDisplayValueAsText(ContentBundleEditorPin->GetStatus());
							}

							return UEnum::GetDisplayValueAsText(EContentBundleStatus::Unknown);
						})
					.ColorAndOpacity(MakeAttributeLambda([ContentBundleTreeItem] { return ContentBundleTreeItem->GetItemColor(); }))
				];
		}
	}
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE