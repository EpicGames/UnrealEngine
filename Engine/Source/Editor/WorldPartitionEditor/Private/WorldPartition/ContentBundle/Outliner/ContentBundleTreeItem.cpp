// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/ContentBundle/Outliner/ContentBundleTreeItem.h"

#include "WorldPartition/ContentBundle/ContentBundle.h"
#include "WorldPartition/ContentBundle/ContentBundleEditor.h"
#include "WorldPartition/ContentBundle/ContentBundleEditorSubsystem.h"
#include "WorldPartition/ContentBundle/Outliner/ContentBundleMode.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ContentBundle"

const FSceneOutlinerTreeItemType FContentBundleTreeItem::Type(&ISceneOutlinerTreeItem::Type);

FContentBundleTreeItem::FContentBundleTreeItem(const FInitializationValues& InitializationValues)
	: ISceneOutlinerTreeItem(Type)
	, ContentBundleEditor(InitializationValues.ContentBundleEditor)
	, Mode(InitializationValues.Mode)
{
	
}

bool FContentBundleTreeItem::IsValid() const
{
	return ContentBundleEditor.IsValid();
}

FSceneOutlinerTreeItemID FContentBundleTreeItem::GetID() const
{
	TSharedPtr<FContentBundleEditor> ContentBundleEditorPin = ContentBundleEditor.Pin();
	if (ContentBundleEditorPin != nullptr)
	{
		return FSceneOutlinerTreeItemID(ContentBundleEditorPin->GetTreeItemID());
	}

	return FSceneOutlinerTreeItemID();
}

FString FContentBundleTreeItem::GetDisplayString() const
{
	TSharedPtr<FContentBundleEditor> ContentBundleEditorPin = ContentBundleEditor.Pin();
	if (ContentBundleEditorPin != nullptr)
	{
		return FString::Printf(TEXT("%s%s"), *ContentBundleEditorPin->GetDisplayName(), ContentBundleEditorPin->IsBeingEdited() ? TEXT(" (Current)") : TEXT(""));
	}

	return LOCTEXT("ContentBundleMissingForTreeItem", "(Invalid Content Bundle)").ToString();
}

bool FContentBundleTreeItem::CanInteract() const
{
	TSharedPtr<FContentBundleEditor> ContentBundleEditorPin = ContentBundleEditor.Pin();
	if (ContentBundleEditorPin != nullptr)
	{
		return ContentBundleEditorPin->GetInjectedWorld() == Mode.GetEditingWorld();
	}
	
	return false;
}

TSharedRef<SWidget> FContentBundleTreeItem::GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(MakeAttributeLambda([this] { return  FText::FromString(GetDisplayString()); }))
			.ColorAndOpacity(MakeAttributeLambda([this] 
			{ 
				return GetItemColor();
			}))
		];
}

FSlateColor FContentBundleTreeItem::GetItemColor() const
{
	TSharedPtr<FContentBundleEditor> ContentBundleEditorPin = ContentBundleEditor.Pin();
	if (ContentBundleEditorPin != nullptr)
	{
		if (!CanInteract())
		{
			return FSlateColor(FSceneOutlinerCommonLabelData::DarkColor);
		}

		if (ContentBundleEditorPin->IsBeingEdited())
		{
			return FAppStyle::Get().GetSlateColor("Colors.AccentGreen");
		}
	}

	return FSlateColor::UseForeground();
}

#undef LOCTEXT_NAMESPACE