// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/ContentBundle/Outliner/SContentBundleOutliner.h"

#include "WorldPartition/ContentBundle/ContentBundleEditor.h"

void SContentBundleOutliner::SelectContentBundle(const TWeakPtr<FContentBundleEditor>& ContentBundle)
{
	TSharedPtr<FContentBundleEditor> ContentBundleEditorPin = ContentBundle.Pin();
	if (ContentBundleEditorPin != nullptr)
	{
		FSceneOutlinerTreeItemPtr Item = GetTreeItem(FSceneOutlinerTreeItemID(ContentBundleEditorPin->GetTreeItemID()));
		if (Item->IsValid())
		{
			SetItemSelection(Item, true, ESelectInfo::Direct);
		}
	}	
}