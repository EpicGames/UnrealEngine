// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/ContentBundle/SContentBundleBrowser.h"
#include "Framework/Application/SlateApplication.h"
#include "SceneOutlinerPublicTypes.h"
#include "WorldPartition/ContentBundle/Outliner/SContentBundleOutliner.h"
#include "WorldPartition/ContentBundle/Outliner/ContentBundleMode.h"
#include "WorldPartition/ContentBundle/Outliner/ContentBundleEditingColumn.h"
#include "WorldPartition/ContentBundle/Outliner/ContentBundleStatusColumn.h"
#include "WorldPartition/ContentBundle/Outliner/ContentBundleClientColumn.h"
#include "WorldPartition/ContentBundle/Outliner/ContentBundleActorCountColumn.h"
#include "WorldPartition/WorldPartition.h"

namespace ContentBundleOutlinerPrivate
{
	enum EColumnPriority
	{
		Client = 0,
		BuiltIn,
		Status,
		Editing,
		ActorCount
	};
}


void SContentBundleBrowser::Construct(const FArguments& InArgs)
{
	FSceneOutlinerInitializationOptions InitOptions;
	InitOptions.bShowHeaderRow = true;
	InitOptions.bShowParentTree = true;
	InitOptions.bShowCreateNewFolder = false;
	InitOptions.bShowTransient = true;
	InitOptions.ModeFactory = FCreateSceneOutlinerMode::CreateLambda([this](SSceneOutliner* Outliner) { return new FContentBundleMode(FContentBundleModeCreationParams(Outliner)); });
	InitOptions.ColumnMap.Add(FContentBundleOutlinerClientColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, ContentBundleOutlinerPrivate::EColumnPriority::Client, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShareable(new FContentBundleOutlinerClientColumn(InSceneOutliner)); }), true));
	InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, ContentBundleOutlinerPrivate::EColumnPriority::BuiltIn));
	InitOptions.ColumnMap.Add(FContentBundleOutlinerStatusColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, ContentBundleOutlinerPrivate::EColumnPriority::Status, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShareable(new FContentBundleOutlinerStatusColumn(InSceneOutliner)); }), false));
	InitOptions.ColumnMap.Add(FContentBundleOutlinerEditingColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, ContentBundleOutlinerPrivate::EColumnPriority::Editing, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShareable(new FContentBundleOutlinerEditingColumn(InSceneOutliner)); }), false));
	InitOptions.ColumnMap.Add(FContentBundleOutlinerActorCountColumn::GetID(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, ContentBundleOutlinerPrivate::EColumnPriority::ActorCount, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShareable(new FContentBundleOutlinerActorCountColumn(InSceneOutliner)); }), false));
	
	ContentBundleOutliner = SNew(SContentBundleOutliner, InitOptions)
		.IsEnabled(FSlateApplication::Get().GetNormalExecutionAttribute());

	ChildSlot
	[
		SAssignNew(ContentAreaBox, SVerticalBox)
		.IsEnabled_Lambda([]()
		{
			UWorldPartition* WorldPartition = GWorld ? GWorld->GetWorldPartition() : nullptr;
			return WorldPartition && WorldPartition->IsContentBundleEnabled();
		})
	];

	ContentAreaBox->AddSlot()
		.AutoHeight()
		.FillHeight(1.0f)
		[
			ContentBundleOutliner.ToSharedRef()
		];
}

void SContentBundleBrowser::SelectContentBundle(const TWeakPtr<FContentBundleEditor>& ContentBundle)
{
	ContentBundleOutliner->SelectContentBundle(ContentBundle);
}