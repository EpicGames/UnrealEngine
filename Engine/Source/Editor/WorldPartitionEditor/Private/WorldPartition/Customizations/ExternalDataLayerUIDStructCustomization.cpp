// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExternalDataLayerUIDStructCustomization.h"
#include "DetailWidgetRow.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "WorldPartition/DataLayer/ExternalDataLayerAsset.h"

#define LOCTEXT_NAMESPACE "ExternalDataLayerUIDStructCustomization"

void FExternalDataLayerUIDStructCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			// text box
			SNew(STextBlock)
			.IsEnabled(false)
			.Text_Lambda([StructPropertyHandle]()
			{
				TArray<void*> RawData;
				StructPropertyHandle->AccessRawData(RawData);
				if (RawData.Num() != 1)
				{
					return LOCTEXT("MultipleValues", "Multiple Values");
				}
				if (RawData[0] == nullptr)
				{
					return FText::GetEmpty();
				}
				return FText::FromString(*((FExternalDataLayerUID*)RawData[0])->ToString());
			})
		]
	];
}

#undef LOCTEXT_NAMESPACE