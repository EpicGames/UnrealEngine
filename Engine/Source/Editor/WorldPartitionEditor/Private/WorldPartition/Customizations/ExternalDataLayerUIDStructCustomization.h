// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class FExternalDataLayerUIDStructCustomization : public IPropertyTypeCustomization
{
public:

	/**
	 * Creates a new instance.
	 *
	 * @return A new struct customization for ExternalDataLayerUID.
	 */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FExternalDataLayerUIDStructCustomization);
	}

	// IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}
};