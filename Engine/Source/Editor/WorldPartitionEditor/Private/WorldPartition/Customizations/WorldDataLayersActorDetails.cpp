// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldDataLayersActorDetails.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"

#define LOCTEXT_NAMESPACE "WorldDataLayersActorDetails"

TSharedRef<IDetailCustomization> FWorldDataLayersActorDetails::MakeInstance()
{
	return MakeShareable( new FWorldDataLayersActorDetails);
}

void FWorldDataLayersActorDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	TArray<TWeakObjectPtr<UObject>> CustomizedObjects;
	DetailBuilder.GetObjectsBeingCustomized(CustomizedObjects);
	if (AWorldDataLayers* CustomizedWorldDataLayers = (CustomizedObjects.Num() == 1) ? Cast<AWorldDataLayers>(CustomizedObjects[0]) : nullptr)
	{
		if (ULevel* CustomizedLevel = CustomizedWorldDataLayers->GetLevel())
		{
			IDetailCategoryBuilder& WorldCategory = DetailBuilder.EditCategory("World");
			WorldCategory.AddCustomRow(LOCTEXT("UseExternalPackageDataLayerInstancesRow", "UseExternalPackageDataLayerInstances"), true)
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("UseExternalPackageDataLayerInstances", "Use External Package Data Layer Instances"))
				.ToolTipText(LOCTEXT("UseExternalPackageDataLayerInstances_ToolTip", "Use External Package Data Layer Instances, data layer instances will be stored in their own external package."))
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.IsEnabled(this, &FWorldDataLayersActorDetails::UseExternalPackageDataLayerInstancesEnabled, CustomizedWorldDataLayers)
			]
			.ValueContent()
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &FWorldDataLayersActorDetails::OnUseExternalPackageDataLayerInstancesChanged, CustomizedWorldDataLayers)
				.IsChecked(this, &FWorldDataLayersActorDetails::IsUseExternalPackageDataLayerInstancesChecked, CustomizedWorldDataLayers)
				.IsEnabled(this, &FWorldDataLayersActorDetails::UseExternalPackageDataLayerInstancesEnabled, CustomizedWorldDataLayers)
			];
		}
	}
}

bool FWorldDataLayersActorDetails::UseExternalPackageDataLayerInstancesEnabled(AWorldDataLayers* WorldDataLayers) const
{
	return WorldDataLayers && WorldDataLayers->SupportsExternalPackageDataLayerInstances();
}

ECheckBoxState FWorldDataLayersActorDetails::IsUseExternalPackageDataLayerInstancesChecked(AWorldDataLayers* WorldDataLayers) const
{
	return WorldDataLayers && WorldDataLayers->IsUsingExternalPackageDataLayerInstances() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FWorldDataLayersActorDetails::OnUseExternalPackageDataLayerInstancesChanged(ECheckBoxState BoxState, AWorldDataLayers* WorldDataLayers)
{
	if (WorldDataLayers)
	{
		WorldDataLayers->SetUseExternalPackageDataLayerInstances(BoxState == ECheckBoxState::Checked, /*bInteractiveMode*/ true);
	}
}

#undef LOCTEXT_NAMESPACE
