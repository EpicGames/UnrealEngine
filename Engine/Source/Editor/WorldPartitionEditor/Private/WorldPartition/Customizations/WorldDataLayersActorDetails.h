// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"
#include "Styling/SlateTypes.h"

class IDetailLayoutBuilder;
class AWorldDataLayers;

class FWorldDataLayersActorDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;

private:
	bool UseExternalPackageDataLayerInstancesEnabled(AWorldDataLayers* WorldDataLayers) const;
	ECheckBoxState IsUseExternalPackageDataLayerInstancesChecked(AWorldDataLayers* WorldDataLayers) const;
	void OnUseExternalPackageDataLayerInstancesChanged(ECheckBoxState BoxState, AWorldDataLayers* WorldDataLayers);
};
