// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartitionDetailsCustomization.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/WorldPartitionEditorPerProjectUserSettings.h"
#include "WorldPartition/WorldPartitionEditorSettings.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"
#include "Misc/MessageDialog.h"
#include "SEnumCombo.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "IDocumentation.h"

#define LOCTEXT_NAMESPACE "FWorldPartitionDetails"

TSharedRef<IDetailCustomization> FWorldPartitionDetails::MakeInstance()
{
	return MakeShareable(new FWorldPartitionDetails);
}

void FWorldPartitionDetails::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	InDetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	check(ObjectsBeingCustomized.Num() == 1);

	DetailBuilder = &InDetailBuilder;
	WorldPartition = CastChecked<UWorldPartition>(ObjectsBeingCustomized[0].Get());
	RuntimeHashClass = (WorldPartition.IsValid() && WorldPartition->RuntimeHash) ? WorldPartition->RuntimeHash->GetClass() : nullptr;

	IDetailCategoryBuilder& WorldPartitionCategory = InDetailBuilder.EditCategory("WorldPartitionSetup");

	// Enable streaming checkbox
	WorldPartitionCategory.AddCustomRow(LOCTEXT("EnableStreaming", "Enable Streaming"), false)
		.RowTag(TEXT("EnableStreaming"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("WorldPartitionEnableStreaming", "Enable Streaming"))
			.ToolTipText(LOCTEXT("WorldPartitionEnableStreaming_ToolTip", "Set the world partition enable streaming state."))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(MakeAttributeLambda([this]() { return WorldPartition.IsValid() && WorldPartition->IsStreamingEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; }))
			.OnCheckStateChanged(this, &FWorldPartitionDetails::HandleWorldPartitionEnableStreamingChanged)
		]
		.Visibility(TAttribute<EVisibility>::CreateLambda([this]() { return WorldPartition.IsValid() && WorldPartition->SupportsStreaming() ? EVisibility::Visible : EVisibility::Hidden; }));

	// Disable world partition button
	if (GetDefault<UWorldPartitionEditorSettings>()->GetAdvancedMode())
	{
		// Default world partition settings
		WorldPartitionCategory.AddCustomRow(LOCTEXT("DefaultWorldPartitionSettingsRow", "DefaultWorldPartitionSettings"), true)
			.NameContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DisableWorldPartition", "Disable World Partition"))
				.ToolTipText(LOCTEXT("DisableWorldPartition_ToolTip", "Disable World Partition for this world, only works if streaming is disabled."))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(SButton)
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.IsEnabled_Lambda([this]()
					{
						return WorldPartition.IsValid() && !WorldPartition->IsStreamingEnabled();
					})
					.OnClicked_Lambda([this]()
					{
						if (FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("RemoveWorldPartitionConfirmation", "You are about to remove world partition from the current level. Continue?")) == EAppReturnType::Yes)
						{
							FScopedTransaction Transaction(LOCTEXT("RemoveWorldPartition", "Remove World Partition"));
							UWorldPartition::RemoveWorldPartition(WorldPartition->GetWorld()->GetWorldSettings());
						}
						return FReply::Handled();
					})
					[
						SNew(STextBlock)
						.Text(LOCTEXT("DisableButtom", "Disable"))
						.ToolTipText(LOCTEXT("DisableWorldPartition_ToolTip", "Disable World Partition for this world, only works if streaming is disabled."))
						.Font(IDetailLayoutBuilder::GetDetailFont())
					]
				]
			];
	}

	// Runtime hash class selector
	WorldPartitionCategory.AddCustomRow(LOCTEXT("RuntimeHashClass", "Runtime Hash Class"), false)
		.RowTag(TEXT("RuntimeHashClass"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("WorldPartitionRuntimeHashClass", "Runtime Hash Class"))
			.ToolTipText(LOCTEXT("WorldPartitionRuntimeHashClass_ToolTip", "Set the world partition runtime hash class."))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SClassPropertyEntryBox)
			.MetaClass(UWorldPartitionRuntimeHash::StaticClass())
			.AllowNone(false)
			.HideViewOptions(true)
			.SelectedClass_Lambda([this]() { return RuntimeHashClass; })
			.OnSetClass_Lambda([this](const UClass* Class) { HandleWorldPartitionRuntimeHashClassChanged(Class); })
		]
		.Visibility(TAttribute<EVisibility>::CreateLambda([this]() { return WorldPartition.IsValid() && WorldPartition->IsStreamingEnabled() ? EVisibility::Visible : EVisibility::Hidden; }));

	// Runtime hash properties
	if (WorldPartition->RuntimeHash)
	{
		FAddPropertyParams Params;
		Params.HideRootObjectNode(true);
		Params.UniqueId(TEXT("RuntimeHash"));

		if (IDetailPropertyRow* RuntimeHashRow = WorldPartitionCategory.AddExternalObjects({ WorldPartition->RuntimeHash }, EPropertyLocation::Default, Params))
		{
			(*RuntimeHashRow)
				.ShouldAutoExpand(true)
				.DisplayName(LOCTEXT("RuntimeHash", "Runtime Hash"))
				.Visibility(TAttribute<EVisibility>::CreateLambda([this]() { return WorldPartition.IsValid() && WorldPartition->IsStreamingEnabled() ? EVisibility::Visible : EVisibility::Hidden; }));
		}

		if (!WorldPartition->RuntimeHash->SupportsHLODs())
		{
			InDetailBuilder.HideProperty(InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UWorldPartition, DefaultHLODLayer), UWorldPartition::StaticClass()));
		}
	}

	{
		// Custom editor for UWorldPartitionEditorPerProjectUserSettings::bForceDisableDynamicLoadingRangeScaling, so it is easier for users to find than going into editor settings
		WorldPartitionCategory.AddCustomRow(LOCTEXT("LocallyDisableDynamicLoadingRangeScaling", "Locally Disable Dynamic Loading Range Scaling"), true)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("LocallyDisableDynamicLoadingRangeScaling", "Locally Disable Dynamic Loading Range Scaling"))
			.ToolTipText(LOCTEXT("LocallyDisableDynamicLoadingRangeScaling_ToolTip", "Disable Dynamic Loading Range Scaling in local user settings, without modifying the map and affecting other users."))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(MakeAttributeLambda([this]()
			{
				if (WorldPartition.IsValid())
				{
					return GetDefault<UWorldPartitionEditorPerProjectUserSettings>()->GetForceDisableDynamicLoadingRangeScaling(WorldPartition->GetWorld()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}
				return ECheckBoxState::Unchecked;
			}))
			.OnCheckStateChanged_Lambda([this](ECheckBoxState InCheckBoxState)
			{
				if (WorldPartition.IsValid())
				{
					GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>()->SetForceDisableDynamicLoadingRangeScaling(WorldPartition->GetWorld(), InCheckBoxState == ECheckBoxState::Checked);
				}
			})
		]
		.Visibility(TAttribute<EVisibility>::CreateLambda([this]() { return WorldPartition.IsValid() && WorldPartition->SupportsStreaming() ? EVisibility::Visible : EVisibility::Hidden; }));

		// Custom editor for AWorldSettings::DynamicLoadingRangeScaling, to allow disabling it on a complex condition
		UWorld* World = WorldPartition->GetWorld();
		AWorldSettings* WorldSettings = (World != nullptr) ? World->GetWorldSettings() : nullptr;
		if (WorldSettings != nullptr)
		{
			const TSharedPtr<FStructOnScope> DataView = MakeShared<FStructOnScope>(FDynamicLoadingRangeScaling::StaticStruct(), reinterpret_cast<uint8*>(&WorldSettings->DynamicLoadingRangeScaling));
			FAddPropertyParams AddParams;
			AddParams.CreateCategoryNodes(true);
			if (IDetailPropertyRow* PropertyRow = WorldPartitionCategory.AddExternalStructureProperty(DataView, NAME_None, EPropertyLocation::Advanced, AddParams))
			{
				PropertyRow->IsEnabled(TAttribute<bool>::CreateLambda([this]()
				{
					if (WorldPartition.IsValid())
					{
						return !GetDefault<UWorldPartitionEditorPerProjectUserSettings>()->GetForceDisableDynamicLoadingRangeScaling(WorldPartition->GetWorld());
					}
					return false;
				}));
				PropertyRow->Visibility(TAttribute<EVisibility>::CreateLambda([this]() { return WorldPartition.IsValid() && WorldPartition->SupportsStreaming() ? EVisibility::Visible : EVisibility::Hidden; }));
				PropertyRow->ShouldAutoExpand(false);
				PropertyRow->DisplayName(LOCTEXT("DynamicLoadingRangeScaling", "Dynamic Loading Range Scaling"));
				PropertyRow->ToolTip(LOCTEXT("DynamicLoadingRangeScaling_ToolTip", "Locally force disable the dynamic loading range scaling - stored in local per-user settings."));

				TSharedPtr<IPropertyHandle> PropertyHandle = PropertyRow->GetPropertyHandle();
				PropertyHandle->SetOnChildPropertyValuePreChange(FSimpleDelegate::CreateLambda([this]()
				{
					if (WorldPartition.IsValid() && WorldPartition->GetWorld())
					{
						// Handle undo and dirty
						WorldPartition->GetWorld()->GetWorldSettings()->Modify();
					}
				}));
			}
		}
	}
}

void FWorldPartitionDetails::HandleWorldPartitionEnableStreamingChanged(ECheckBoxState InCheckState)
{
	if (InCheckState == ECheckBoxState::Checked)
	{
		if (!WorldPartition->bStreamingWasEnabled)
		{
			if (FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("WorldPartitionConfirmEnableStreaming", "You are about to enable streaming for the first time, the world will be setup to stream. Continue?")) == EAppReturnType::No)
			{
				return;
			}

			if (FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("WorldPartitionEnableStreamingDialolg", "Please refer to our documentation for how to set up streaming.\n\nWould you like to open it now? ")) == EAppReturnType::Yes)
			{
				IDocumentation::Get()->Open(TEXT("world-partition-in-unreal-engine"), FDocumentationSourceInfo(TEXT("worldpartition")));
			}

			WorldPartition->bStreamingWasEnabled = true;
		}

		WorldPartition->SetEnableStreaming(true);
	}
	else
	{
		if (FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("WorldPartitionConfirmDisableStreaming", "You are about to disable streaming, all actors in the world will be always loaded. Continue?")) == EAppReturnType::Yes)
		{
			WorldPartition->SetEnableStreaming(false);
		}
	}
}

void FWorldPartitionDetails::HandleWorldPartitionRuntimeHashClassChanged(const UClass* InRuntimeHashClass)
{
	check(InRuntimeHashClass);
	check(InRuntimeHashClass->IsChildOf<UWorldPartitionRuntimeHash>());
	check(!(InRuntimeHashClass->GetClassFlags() & CLASS_Abstract));

	RuntimeHashClass = InRuntimeHashClass;

	if (WorldPartition.IsValid() && (!WorldPartition->RuntimeHash || WorldPartition->RuntimeHash->GetClass() != RuntimeHashClass))
	{
		WorldPartition->Modify();
		WorldPartition->RuntimeHash = UWorldPartitionRuntimeHash::ConvertWorldPartitionHash(WorldPartition->RuntimeHash, RuntimeHashClass);
		WorldPartition->WorldPartitionChangedEvent.Broadcast(WorldPartition->GetWorld());

		DetailBuilder->ForceRefreshDetails();
	}
}

#undef LOCTEXT_NAMESPACE
