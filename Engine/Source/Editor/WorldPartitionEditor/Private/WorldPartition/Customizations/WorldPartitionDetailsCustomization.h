// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "IDetailCustomization.h"
#include "WorldPartition/WorldPartition.h"

class IDetailLayoutBuilder;

enum class ECheckBoxState : uint8;

class FWorldPartitionDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	FWorldPartitionDetails()
	{}

private:
	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;

	// Callback for changes in the world partition enable streaming.
	void HandleWorldPartitionEnableStreamingChanged(ECheckBoxState InCheckState);

	// Callback for changes in the world partition runtime hash class.
	void HandleWorldPartitionRuntimeHashClassChanged(const UClass* InRuntimeHashClass);

	IDetailLayoutBuilder* DetailBuilder;
	TWeakObjectPtr<UWorldPartition> WorldPartition;
	const UClass* RuntimeHashClass;
};