// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartitionEditorPerProjectUserSettingsDetails.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "WorldPartition/WorldPartitionEditorPerProjectUserSettings.h"
#include "Misc/MessageDialog.h"

#define LOCTEXT_NAMESPACE "FWorldPartitionEditorPerProjectUserSettingsCustomization"

TSharedRef<IDetailCustomization> FWorldPartitionEditorPerProjectUserSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FWorldPartitionEditorPerProjectUserSettingsCustomization);
}

FReply FWorldPartitionEditorPerProjectUserSettingsCustomization::OnClearPerWorldSettings()
{
	const FText Message = LOCTEXT("ClearPerWorldSettingsWarningMessage", "This will Clear all local Per World Settings: Loaded Regions, Loaded Location Volumes, Loaded/Visibility Data Layers states.");
	const FText Title = LOCTEXT("ClearPerWorldSettingsWarningMessageTitle", "Continue?");
	if (FMessageDialog::Open(EAppMsgType::YesNo, Message, Title) == EAppReturnType::Yes)
	{
		GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>()->ClearPerWorldSettings();
	}
	return FReply::Handled();
}

void FWorldPartitionEditorPerProjectUserSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	IDetailCategoryBuilder& Category = InDetailBuilder.EditCategory("Default");
	Category.AddCustomRow(LOCTEXT("PerWorldSettingsRow", "Per World Settings"), false)
	.NameContent()
	[
		SNew(STextBlock)
		.Text(LOCTEXT("PerWorldSettingsText", "Per World Settings"))
		.Font(InDetailBuilder.GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		[
			SNew(SButton)
			.Text(LOCTEXT("ClearPerWorldSettingsText", "Clear"))
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.ToolTipText(LOCTEXT("ClearPerWorldSettingsTooltipText", "Clear Per World Settings (Loaded Regions, Loaded Location Volumes, Loaded/Visibility Data Layers states)"))
			.OnClicked(this, &FWorldPartitionEditorPerProjectUserSettingsCustomization::OnClearPerWorldSettings)
		]
	];
}

#undef LOCTEXT_NAMESPACE