// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "IDetailCustomization.h"

class FWorldPartitionEditorPerProjectUserSettingsCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	FWorldPartitionEditorPerProjectUserSettingsCustomization() {}

	FReply OnClearPerWorldSettings();

private:
	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder);
};