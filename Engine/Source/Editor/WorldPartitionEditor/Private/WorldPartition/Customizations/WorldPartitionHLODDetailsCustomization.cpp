// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartitionHLODDetailsCustomization.h"

#include "Algo/AnyOf.h"
#include "Algo/Copy.h"
#include "Algo/ForEach.h"
#include "Algo/Transform.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Components/StaticMeshComponent.h"
#include "ContentBrowserModule.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Dialogs/DlgPickPath.h"
#include "Editor.h"
#include "Engine/Selection.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IContentBrowserDataModule.h"
#include "IContentBrowserSingleton.h"
#include "Interfaces/IMainFrameModule.h"
#include "IStructureDetailsView.h"
#include "MaterialShared.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "PropertyHandle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/SHLODCompareWindow.h"
#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "FWorldPartitionHLODDetailsCustomization"

TSharedRef<IDetailCustomization> FWorldPartitionHLODDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FWorldPartitionHLODDetailsCustomization);
}

void FWorldPartitionHLODDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayoutBuilder)
{
	SelectedObjects = DetailLayoutBuilder.GetSelectedObjects();

	IDetailCategoryBuilder& PluginCategory = DetailLayoutBuilder.EditCategory("Tools");
	PluginCategory.AddCustomRow(LOCTEXT("HLODTools", "HLOD Tools"), false)
	.RowTag("HLOD_Tools")
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			[
				SNew(SButton)
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.Text(LOCTEXT("BuildHLODButtonText", "Build HLOD"))
				.OnClicked(this, &FWorldPartitionHLODDetailsCustomization::OnBuildHLOD)
				.IsEnabled(this, &FWorldPartitionHLODDetailsCustomization::CanBuildHLOD)
			]
			+ SHorizontalBox::Slot()
			[
				SNew(SButton)
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.VAlign(EVerticalAlignment::VAlign_Center)
				.Text(LOCTEXT("ForceBuildHLODButtonText", "Force Build HLOD"))
				.OnClicked(this, &FWorldPartitionHLODDetailsCustomization::OnForceBuildHLOD)
				.IsEnabled(this, &FWorldPartitionHLODDetailsCustomization::CanBuildHLOD)
			]
		]
		+ SVerticalBox::Slot()
		[
			SNew(SButton)
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Text(LOCTEXT("ExportAssetsButtonText", "Export HLOD Assets"))
			.OnClicked(this, &FWorldPartitionHLODDetailsCustomization::OnExportAssets)
			.IsEnabled(this, &FWorldPartitionHLODDetailsCustomization::CanExportAssets)
		]
		+ SVerticalBox::Slot()
		[
			SNew(SButton)
			.HAlign(EHorizontalAlignment::HAlign_Center)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Text(LOCTEXT("CompareHLODButtonText", "Compare Against Source"))
			.ToolTipText(LOCTEXT("CompareHLODButtonTooltip", "Open a wipe comparison of the HLOD against its source actors"))
			.OnClicked(this, &FWorldPartitionHLODDetailsCustomization::OnCompareHLOD)
			.IsEnabled(this, &FWorldPartitionHLODDetailsCustomization::CanCompareHLOD)
		]
	];
}

TArray<AWorldPartitionHLOD*> FWorldPartitionHLODDetailsCustomization::GetSelectedHLODActors() const
{
	TArray<AWorldPartitionHLOD*> SelectedHLODActors;
	SelectedHLODActors.Reserve(SelectedObjects.Num());

	auto IsA_WPHLOD = [](TWeakObjectPtr<UObject> InObject) { return InObject.Get() && InObject->IsA<AWorldPartitionHLOD>(); };
	auto Cast_WPHLOD = [](TWeakObjectPtr<UObject> InObject) { return Cast<AWorldPartitionHLOD>(InObject); };
	Algo::TransformIf(SelectedObjects, SelectedHLODActors, IsA_WPHLOD, Cast_WPHLOD);

	return SelectedHLODActors;
}

bool FWorldPartitionHLODDetailsCustomization::CanBuildHLOD() const
{
	TArray<AWorldPartitionHLOD*> SelectedHLODActors = GetSelectedHLODActors();

	for (AWorldPartitionHLOD* HLODActor : SelectedHLODActors)
	{
		if (HLODActor == nullptr)
		{
			return false;
		}

		if (HLODActor->IsTemplate())
		{
			return false;
		}

		if (HLODActor->GetSourceActors() == nullptr)
		{
			return false;
		}
	}

	return true;
}

FReply FWorldPartitionHLODDetailsCustomization::OnBuildHLOD()
{
	return OnBuildHLODInternal(/*bForceBuild=*/false);
}

FReply FWorldPartitionHLODDetailsCustomization::OnForceBuildHLOD()
{
	return OnBuildHLODInternal(/*bForceBuild=*/true);
}

FReply FWorldPartitionHLODDetailsCustomization::OnBuildHLODInternal(bool bForceBuild)
{
	TArray<AWorldPartitionHLOD*> SelectedHLODActors = GetSelectedHLODActors();

	// Gather all components
	TArray<UActorComponent*> ActorComponents;
	Algo::ForEach(SelectedHLODActors, [&ActorComponents](AWorldPartitionHLOD* HLODActor) { ActorComponents.Append(HLODActor->GetComponents().Array()); });

	// Recreate render states for all the components we're about to process
	FGlobalComponentRecreateRenderStateContext RecreateRenderStateContext(ActorComponents);

	// Use a material update context to signal any change made to materials to the render thread... Exclude the recreate render state flag as this step is already performed
	// by the FGlobalComponentRecreateRenderStateContext above.
	FMaterialUpdateContext MaterialUpdateContext(FMaterialUpdateContext::EOptions::Default & ~FMaterialUpdateContext::EOptions::RecreateRenderStates);
	for (UActorComponent* ActorComponent : ActorComponents)
	{
		if (UStaticMeshComponent* SMComponent = Cast<UStaticMeshComponent>(ActorComponent))
		{
			TArray<UMaterialInterface*> UsedMaterials;
			SMComponent->GetUsedMaterials(UsedMaterials);

			// Add all used materials to the material update context.
			for (UMaterialInterface* UsedMaterial : UsedMaterials)
			{
				if (UsedMaterial)
				{
					MaterialUpdateContext.AddMaterialInterface(UsedMaterial);
				}
			}
		}
	}

	FScopedSlowTask Progress(SelectedHLODActors.Num(), LOCTEXT("BuildingHLODsforActors", "Building HLODs for actors..."));
	Progress.MakeDialog(true);

	int32 NumHLODActorsUpdated = 0;

	// Build HLODs
	for (int32 Index = 0; Index < SelectedHLODActors.Num(); ++Index)
	{
		Progress.EnterProgressFrame(1.0f, FText::Format(LOCTEXT("BuildingHLODsforActorProgress", "Building HLODs ({0}/{1})"), Index + 1, SelectedHLODActors.Num()));
		if (Progress.ShouldCancel())
		{
			break;
		}

		AWorldPartitionHLOD* HLODActor = SelectedHLODActors[Index];
		bool bUpdated = HLODActor->BuildHLOD(bForceBuild);
		NumHLODActorsUpdated += bUpdated ? 1 : 0;
	};

	// Toast
	FNotificationInfo Notification(LOCTEXT("HLODBuildCompleted", "HLOD Build Completed"));
	Notification.Image = FAppStyle::GetBrush(TEXT("Icons.Info"));
	Notification.bFireAndForget = true;
	Notification.ExpireDuration = 3.0f;
	Notification.bUseThrobber = true;

	FText NumHLODActorsUpdatedText = NumHLODActorsUpdated > 0 ? FText::AsNumber(NumHLODActorsUpdated) : LOCTEXT("NoHLODActorUpdated", "No");
	Notification.SubText = FText::Format(LOCTEXT("NumHLODActorsUpdated", "{0} HLOD {1}|plural(one=actor,other=actors) updated"), NumHLODActorsUpdatedText, NumHLODActorsUpdated);
	FSlateNotificationManager::Get().AddNotification(Notification);

	// Force refresh the UI so that any change to the selected actors are properly shown to the user
	GEditor->NoteSelectionChange();

	return FReply::Handled();
}

bool FWorldPartitionHLODDetailsCustomization::CanExportHLODAssets(const AWorldPartitionHLOD* HLODActor)
{
	if (HLODActor->IsTemplate())
	{
		return false;
	}

	if (HLODActor->GetSourceActors() == nullptr)
	{
		return false;
	}

	if (HLODActor->GetHLODResourcesPackage() == nullptr)
	{
		return false;
	}

	bool bHasPrivateStaticMesh = false;
	ForEachObjectWithPackage(HLODActor->GetHLODResourcesPackage(), [&bHasPrivateStaticMesh](UObject* InObject)
	{
		bHasPrivateStaticMesh = InObject->IsA<UStaticMesh>();
		return !bHasPrivateStaticMesh; // Continue until we find a static mesh
	});

	return bHasPrivateStaticMesh;
}

bool FWorldPartitionHLODDetailsCustomization::CanExportAssets() const
{
	TArray<AWorldPartitionHLOD*> SelectedHLODActors = GetSelectedHLODActors();
	if (SelectedHLODActors.IsEmpty())
	{
		return false;
	}

	return Algo::AnyOf(SelectedHLODActors, CanExportHLODAssets);
}

static EAppReturnType::Type EditExportHLODAssetsParams(FExportHLODAssetsParams& InOutParams, const TArray<AWorldPartitionHLOD*>& InHLODActorsToExport)
{
	// Setup property editor for a struct
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsArgs;
	DetailsArgs.bAllowSearch = false; // no search bar
	DetailsArgs.bShowOptions = false;
	DetailsArgs.bHideSelectionTip = true;
	DetailsArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;

	FStructureDetailsViewArgs StructArgs;
	StructArgs.bShowObjects = false;

	TSharedRef<IStructureDetailsView> StructDetails =
		PropertyEditorModule.CreateStructureDetailView(DetailsArgs, StructArgs, nullptr);

	TSharedRef<FStructOnScope> StructData = MakeShared<FStructOnScope>(FExportHLODAssetsParams::StaticStruct());
	FExportHLODAssetsParams& WorkingCopy = *reinterpret_cast<FExportHLODAssetsParams*>(StructData->GetStructMemory());
	FExportHLODAssetsParams::StaticStruct()->CopyScriptStruct(&WorkingCopy, &InOutParams);
	StructDetails->SetStructureData(StructData);
	
	auto CanExport = [&WorkingCopy, &InHLODActorsToExport](FText& OutExportErrorMessage) -> bool
	{
		OutExportErrorMessage = FText::GetEmpty();

		FExportHLODAssetsParams TestExportHLODAssetsParams = WorkingCopy;
		TestExportHLODAssetsParams.bTestExportOnly = true;

		for (AWorldPartitionHLOD* HLODActor : InHLODActorsToExport)
		{
			FString OutErrorMessage;
			HLODActor->ExportHLODAssets(TestExportHLODAssetsParams, OutErrorMessage);
			if (!OutErrorMessage.IsEmpty())
			{
				OutExportErrorMessage = FText::FromString(OutErrorMessage);
				return false;
			}
		}

		return true;
	};

	FText ExportErrorMessage;
	bool bCanExportCached = CanExport(ExportErrorMessage);

	StructDetails->GetOnFinishedChangingPropertiesDelegate().AddLambda(
		[&bCanExportCached, &CanExport, &ExportErrorMessage](const FPropertyChangedEvent&)
		{
			bCanExportCached = CanExport(ExportErrorMessage);
		});	

	bCanExportCached = CanExport(ExportErrorMessage);

	// Modal window
	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(LOCTEXT("ExportHLODAssetsWindowTitle", "Export HLOD Assets"))
		.ClientSize(FVector2D(560, 400))              
		.SizingRule(ESizingRule::UserSized)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.IsTopmostWindow(true);

	EAppReturnType::Type Result = EAppReturnType::Cancel;

	Window->SetContent(
		SNew(SBorder)
		.Padding(4.f)
		[
			SNew(SVerticalBox)

			// Details view
			+ SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				StructDetails->GetWidget().ToSharedRef()
			]

			// Error label
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 6.f)
			[
				SNew(SBorder)
				.Padding(6.f)
				.BorderImage(FAppStyle::GetBrush("MessageLog.ListBorder"))
				.BorderBackgroundColor(FStyleColors::Error)
				[
					SNew(STextBlock)
					.Text_Lambda([&ExportErrorMessage]() { return ExportErrorMessage; })
					.Visibility_Lambda([&ExportErrorMessage]() { return ExportErrorMessage.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; })
					.ColorAndOpacity(FStyleColors::Foreground)
					.Font(FAppStyle::GetFontStyle("NormalBold"))
					.WrapTextAt(500.f)
					.Justification(ETextJustify::Left)
				]
			]

			// Buttons
			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(0.f, 8.f, 0.f, 0.f)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FMargin(4.f, 0.f))

				// "Export"
				+ SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.IsEnabled_Lambda([&bCanExportCached]() { return bCanExportCached; })
					.OnClicked_Lambda([&Result, Window]()
					{
						Result = EAppReturnType::Ok;
						Window->RequestDestroyWindow();
						return FReply::Handled();
					})
					[
						SNew(SHorizontalBox)

						// Error icon (visible only when error exists)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(SImage)
							.Visibility_Lambda([&ExportErrorMessage]()
							{
								return ExportErrorMessage.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
							})
							.Image(FAppStyle::Get().GetBrush("Icons.Error")) // UE editor’s error icon
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(4.f, 0.f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("Export", "Export"))
						]
					]
				]

				// "Cancel"
				+ SUniformGridPanel::Slot(1, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Text(LOCTEXT("Cancel", "Cancel"))
					.OnClicked_Lambda([Window]()
					{
						Window->RequestDestroyWindow();
						return FReply::Handled();
					})
				]
			]
		]
	);

	IMainFrameModule& MainFrameModule = FModuleManager::GetModuleChecked<IMainFrameModule>("MainFrame");
	FSlateApplication::Get().AddModalWindow(Window, MainFrameModule.GetParentWindow());

	if (Result == EAppReturnType::Ok)
	{
		InOutParams = WorkingCopy; // commit
	}

	return Result;
}

FReply FWorldPartitionHLODDetailsCustomization::OnExportAssets()
{
	TArray<AWorldPartitionHLOD*> HLODActors;
	Algo::CopyIf(GetSelectedHLODActors(), HLODActors, CanExportHLODAssets);

	// Static so that settings persist in the same user session
	static FExportHLODAssetsParams ExportHLODAssetsParams;
	if (EditExportHLODAssetsParams(ExportHLODAssetsParams, HLODActors) != EAppReturnType::Ok)
	{
		return FReply::Handled();
	}
	
	TArray<UObject*> ExportedAssets;
	const uint32 SlowTaskStepsCount = HLODActors.Num() + 1; // +1 for the content browser sync operation...

	FScopedSlowTask SlowTask(SlowTaskStepsCount, FText::FromString(TEXT("Exporting HLOD Assets...")));
	SlowTask.MakeDialogDelayed(1.f, true);

	for (int32 Index = 0; Index < HLODActors.Num(); Index++)
	{
		AWorldPartitionHLOD* HLODActor = HLODActors[Index];
		SlowTask.EnterProgressFrame(1.f, FText::FromString(FString::Printf(TEXT("Exporting HLOD Assets (%d/%d)"), Index + 1, HLODActors.Num())));

		if (SlowTask.ShouldCancel())
		{
			break;
		}

		FString OutErrorMessage;
		ExportedAssets += HLODActor->ExportHLODAssets(ExportHLODAssetsParams, OutErrorMessage);
		if (!OutErrorMessage.IsEmpty())
		{
			FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, FText::FromString(OutErrorMessage), FText::FromString(TEXT("Export failed!")));
			break;
		}			
	}

	if (ExportedAssets.Num())
	{
		SlowTask.EnterProgressFrame(1.f, FText::FromString(TEXT("Finalizing Export...")));

		UContentBrowserDataSubsystem* ContentBrowserDataSubsystem = IContentBrowserDataModule::Get().GetSubsystem();
		
		TArray<FContentBrowserItem> ContentBrowserItems;
		ContentBrowserDataSubsystem->EnumerateItemsForObjects(ExportedAssets, [&ContentBrowserItems](FContentBrowserItemData&& ItemData)
		{
			ContentBrowserItems.Emplace(ItemData);
			return true;
		});

		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		ContentBrowserModule.Get().SyncBrowserToItems(ContentBrowserItems);
	}

	return FReply::Handled();
}

bool FWorldPartitionHLODDetailsCustomization::CanCompareHLOD() const
{
	TArray<AWorldPartitionHLOD*> SelectedHLODActors = GetSelectedHLODActors();
	if (SelectedHLODActors.IsEmpty())
	{
		return false;
	}

	AWorldPartitionHLOD* HLODActor = SelectedHLODActors[0];
	if (!HLODActor || HLODActor->IsTemplate())
	{
		return false;
	}

	if (HLODActor->GetSourceActors() == nullptr)
	{
		return false;
	}

	if (HLODActor->GetInstanceComponents().Num() == 0)
	{
		return false;
	}

	return true;
}

FReply FWorldPartitionHLODDetailsCustomization::OnCompareHLOD()
{
	TArray<AWorldPartitionHLOD*> SelectedHLODActors = GetSelectedHLODActors();
	if (SelectedHLODActors.IsEmpty())
	{
		return FReply::Handled();
	}

	TArray<TWeakObjectPtr<AWorldPartitionHLOD>> WeakHLODActors;
	for (AWorldPartitionHLOD* Actor : SelectedHLODActors)
	{
		if (Actor)
		{
			WeakHLODActors.Add(Actor);
		}
	}

	if (WeakHLODActors.IsEmpty())
	{
		return FReply::Handled();
	}

	FText WindowTitle = (WeakHLODActors.Num() == 1)
		? FText::Format(LOCTEXT("CompareWindowTitleSingle", "Compare HLOD: {0}"), FText::FromString(WeakHLODActors[0]->GetActorLabel()))
		: FText::Format(LOCTEXT("CompareWindowTitleMulti", "Compare HLOD ({0} actors)"), FText::AsNumber(WeakHLODActors.Num()));

	TSharedRef<SWindow> Window = SNew(SWindow)
		.Title(WindowTitle)
		.ClientSize(FVector2D(1280, 720))
		.SizingRule(ESizingRule::UserSized)
		.SupportsMinimize(true)
		.SupportsMaximize(true);

	Window->SetContent(
		SNew(SHLODCompareWindow)
		.HLODActors(WeakHLODActors)
	);

	FSlateApplication::Get().AddWindow(Window);

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
