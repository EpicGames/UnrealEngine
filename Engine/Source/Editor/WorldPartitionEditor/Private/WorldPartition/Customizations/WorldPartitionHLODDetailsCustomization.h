// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "IDetailCustomization.h"

class IDetailLayoutBuilder;
class AWorldPartitionHLOD;

class FWorldPartitionHLODDetailsCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	FWorldPartitionHLODDetailsCustomization()
	{}

private:
	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;

	TArray<AWorldPartitionHLOD*> GetSelectedHLODActors() const;

	bool CanBuildHLOD() const;
	FReply OnBuildHLOD();
	FReply OnForceBuildHLOD();
	FReply OnBuildHLODInternal(bool bForceBuild);

	static bool CanExportHLODAssets(const AWorldPartitionHLOD* HLODActor);
	bool CanExportAssets() const;
	FReply OnExportAssets();

	bool CanCompareHLOD() const;
	FReply OnCompareHLOD();

private:
	TArray<TWeakObjectPtr<UObject>> SelectedObjects;
};
