// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartitionHLODLayerDetailsCustomization.h"
#include "DetailLayoutBuilder.h"

#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionRuntimeSpatialHash.h"

#define LOCTEXT_NAMESPACE "FWorldPartitionHLODLayerDetailsCustomization"

TSharedRef<IDetailCustomization> FWorldPartitionHLODLayerDetailsCustomization::MakeInstance()
{
	return MakeShareable(new FWorldPartitionHLODLayerDetailsCustomization);
}

void FWorldPartitionHLODLayerDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailLayoutBuilder)
{
	UWorldPartition::WorldPartitionChangedEvent.RemoveAll(this);
	UWorldPartition::WorldPartitionChangedEvent.AddSP(this, &FWorldPartitionHLODLayerDetailsCustomization::OnWorldPartitionChanged);

	UHLODLayer* HLODLayer = !InDetailLayoutBuilder.GetSelectedObjects().IsEmpty() ? Cast<UHLODLayer>(InDetailLayoutBuilder.GetSelectedObjects()[0].Get()) : nullptr;
	if (!HLODLayer)
	{
		return;
	}
		
	TSharedPtr<IPropertyHandle> IsSpatiallyLoadedPropertyHandle = InDetailLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UHLODLayer, bIsSpatiallyLoaded), UHLODLayer::StaticClass());
	TSharedPtr<IPropertyHandle> CellSizePropertyHandle = InDetailLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UHLODLayer, CellSize), UHLODLayer::StaticClass());
	TSharedPtr<IPropertyHandle> LoadingRangePropertyHandle = InDetailLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UHLODLayer, LoadingRange), UHLODLayer::StaticClass());
	TSharedPtr<IPropertyHandle> ParentLayerPropertyHandle = InDetailLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UHLODLayer, ParentLayer), UHLODLayer::StaticClass());

	bool bShowLegacyHLODLayerProperties = false;
	if (UWorld* EditorWorld = GEditor->GetEditorWorldContext().World())
	{
		if (UWorldPartition* WorldPartition = EditorWorld->GetWorldPartition())
		{
			bShowLegacyHLODLayerProperties = Cast<UWorldPartitionRuntimeSpatialHash>(WorldPartition->RuntimeHash) != nullptr;
		}
	}
	
	if (!bShowLegacyHLODLayerProperties)
	{
		InDetailLayoutBuilder.HideProperty(IsSpatiallyLoadedPropertyHandle);
		InDetailLayoutBuilder.HideProperty(CellSizePropertyHandle);
		InDetailLayoutBuilder.HideProperty(LoadingRangePropertyHandle);
	}
	else
	{
		IsSpatiallyLoadedPropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FWorldPartitionHLODLayerDetailsCustomization::ForceRefreshLayout));

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const bool bIsSpatiallyLoaded = HLODLayer->IsSpatiallyLoaded();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (!bIsSpatiallyLoaded)
		{
			InDetailLayoutBuilder.HideProperty(CellSizePropertyHandle);
			InDetailLayoutBuilder.HideProperty(LoadingRangePropertyHandle);
			InDetailLayoutBuilder.HideProperty(ParentLayerPropertyHandle);
		}
	}
}

void FWorldPartitionHLODLayerDetailsCustomization::CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailLayoutBuilder)
{
	DetailLayoutBuilderWeakPtr = InDetailLayoutBuilder;
	CustomizeDetails(*InDetailLayoutBuilder);
}


FWorldPartitionHLODLayerDetailsCustomization::~FWorldPartitionHLODLayerDetailsCustomization()
{
	UWorldPartition::WorldPartitionChangedEvent.RemoveAll(this);
}

void FWorldPartitionHLODLayerDetailsCustomization::OnWorldPartitionChanged(UWorld* InWorld)
{
	ForceRefreshLayout();
}

void FWorldPartitionHLODLayerDetailsCustomization::ForceRefreshLayout()
{
	IDetailLayoutBuilder* DetailLayoutBuilder = DetailLayoutBuilderWeakPtr.Pin().Get();
	if (DetailLayoutBuilder)
	{
		DetailLayoutBuilder->ForceRefreshDetails();
	}
}

#undef LOCTEXT_NAMESPACE
