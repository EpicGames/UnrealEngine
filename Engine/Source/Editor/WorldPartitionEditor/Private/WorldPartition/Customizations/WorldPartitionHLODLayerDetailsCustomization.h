// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "IDetailCustomization.h"

class IDetailLayoutBuilder;
class UHLODLayer;

class FWorldPartitionHLODLayerDetailsCustomization : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	~FWorldPartitionHLODLayerDetailsCustomization();

private:
	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailLayoutBuilder) override;
	virtual void CustomizeDetails(const TSharedPtr<IDetailLayoutBuilder>& InDetailLayoutBuilder) override;
	void OnWorldPartitionChanged(UWorld* InWorld);
	void ForceRefreshLayout();

	TWeakPtr<IDetailLayoutBuilder> DetailLayoutBuilderWeakPtr;
};