// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartitionRuntimeHashSetDetailsCustomization.h"
#include "WorldPartition/RuntimeHashSet/RuntimePartitionPersistent.h"
#include "WorldPartition/RuntimeHashSet/WorldPartitionRuntimeHashSet.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "IPropertyUtilities.h"

#define LOCTEXT_NAMESPACE "FWorldPartitionRuntimeHashSetDetails"

TSharedRef<IPropertyTypeCustomization> FRuntimePartitionHLODSetupDetails::MakeInstance()
{
	return MakeShareable(new FRuntimePartitionHLODSetupDetails);
}

void FRuntimePartitionHLODSetupDetails::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle,
	class FDetailWidgetRow& HeaderRow,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow.NameContent()[StructPropertyHandle->CreatePropertyNameWidget()]
			 .ValueContent()[StructPropertyHandle->CreatePropertyValueWidget()];
}

void FRuntimePartitionHLODSetupDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle,
	class IDetailChildrenBuilder& ChildBuilder,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = StructPropertyHandle->GetChildHandle(ChildIndex);
		if (!ChildHandle.IsValid())
			continue;

		bool bShouldAddProperty = true;

		if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FRuntimePartitionHLODSetup, PartitionLayer))
		{
			// Get the UObject pointer value
			UObject* ObjectValue = nullptr;
			if (ChildHandle->GetValue(ObjectValue) == FPropertyAccess::Success && ObjectValue)
			{
				// Replace the original row by its children
				if (!ObjectValue->IsA<URuntimePartitionPersistent>())
				{
					ChildBuilder.AddExternalObjects({ ObjectValue }, FAddPropertyParams().HideRootObjectNode(true).AllowChildren(true));
				}
			}

			bShouldAddProperty = false;
		}
		else if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FRuntimePartitionHLODSetup, Name))
		{
			void* StructAddress = nullptr;
			if (StructPropertyHandle->GetValueData(/*out*/ StructAddress) == FPropertyAccess::Success)
			{
				if (FRuntimePartitionHLODSetup* HLODSetupObject = reinterpret_cast<FRuntimePartitionHLODSetup*>(StructAddress))
				{
					bShouldAddProperty = HLODSetupObject->bIsSpatiallyLoaded;
				}
			}
		}
		else if (ChildHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FRuntimePartitionHLODSetup, RowDisplayName))
		{
			// Property is only used as the "TitlePropery" when showing as an array item
			bShouldAddProperty = false;
		}
		
		if (bShouldAddProperty)
		{
			ChildBuilder.AddProperty(ChildHandle.ToSharedRef());
		}
	}
}

#undef LOCTEXT_NAMESPACE
