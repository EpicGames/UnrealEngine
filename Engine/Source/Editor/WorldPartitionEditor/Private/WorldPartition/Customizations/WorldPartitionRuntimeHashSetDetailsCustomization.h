// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"

struct FRuntimePartitionHLODSetup;

class FRuntimePartitionHLODSetupDetails : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

protected:
	FRuntimePartitionHLODSetupDetails()
	{
	}

private:
	/** IDetailCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle,
		class FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle,
		class IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	FRuntimePartitionHLODSetup* RuntimePartitionHLODSetup;
};
