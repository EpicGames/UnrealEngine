// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartitionRuntimeSpatialHashDetailsCustomization.h"
#include "WorldPartition/WorldPartitionRuntimeSpatialHash.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "FWorldPartitionRuntimeSpatialHashDetails"

TSharedRef<IDetailCustomization> FWorldPartitionRuntimeSpatialHashDetails::MakeInstance()
{
	return MakeShareable(new FWorldPartitionRuntimeSpatialHashDetails);
}

void FWorldPartitionRuntimeSpatialHashDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	check(ObjectsBeingCustomized.Num() == 1);

	WorldPartitionRuntimeSpatialHash = CastChecked<UWorldPartitionRuntimeSpatialHash>(ObjectsBeingCustomized[0].Get());

	IDetailCategoryBuilder& RuntimeSettingsCategory = DetailBuilder.EditCategory("RuntimeSettings");

	RuntimeSettingsCategory.AddCustomRow(LOCTEXT("PreviewGrids", "Preview Grids"), false)
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("WorldPartitionRuntimeSpatialHashPreviewGrids", "Preview Grids"))
			.ToolTipText(LOCTEXT("WorldPartitionRuntimeSpatialHashPreviewGrids_ToolTip", "Toggle preview grids"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked(MakeAttributeLambda([this]() { return WorldPartitionRuntimeSpatialHash->GetPreviewGrids() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; }))
			.OnCheckStateChanged_Lambda([this](ECheckBoxState InState)
			{
				FScopedTransaction Transaction(LOCTEXT("SetRuntimeSpatialHashPreviewGrids", "Change the Runtime Spatial Hash Preview Grids"));
				WorldPartitionRuntimeSpatialHash->SetPreviewGrids(InState == ECheckBoxState::Checked);
			})
		];

	RuntimeSettingsCategory.AddCustomRow(LOCTEXT("PreviewGridLevel", "Preview Grid Level"), false)
		.NameContent()
		[
			SNew(STextBlock)
			.IsEnabled_Lambda([this]() { return WorldPartitionRuntimeSpatialHash->GetPreviewGrids(); })
			.Text(LOCTEXT("WorldPartitionRuntimeSpatialHashPreviewGridLevel", "Preview Grid Level"))
			.ToolTipText(LOCTEXT("WorldPartitionRuntimeSpatialHashPreviewGridLevel_ToolTip", "The grid level to preview"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SNumericEntryBox<int32>)
			.IsEnabled_Lambda([this]() { return WorldPartitionRuntimeSpatialHash->GetPreviewGrids(); })
			.AllowSpin(false)
			.MinValue(0)
			.Value_Lambda([this]() { return WorldPartitionRuntimeSpatialHash->GetPreviewGridLevel(); })
			.OnValueCommitted_Lambda([this](int32 NewValue, ETextCommit::Type CommitType)
			{
				FScopedTransaction Transaction(LOCTEXT("SetRuntimeSpatialHashPreviewGridLevel", "Change the Runtime Spatial Hash Preview Grid Level"));
				WorldPartitionRuntimeSpatialHash->SetPreviewGridLevel(NewValue);
			})
		];
}

#undef LOCTEXT_NAMESPACE
