// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateFwd.h"
#include "IDetailCustomization.h"

class IDetailLayoutBuilder;
class UWorldPartitionRuntimeSpatialHash;

enum class ECheckBoxState : uint8;

class FWorldPartitionRuntimeSpatialHashDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

protected:
	FWorldPartitionRuntimeSpatialHashDetails()
	{}

private:
	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	UWorldPartitionRuntimeSpatialHash* WorldPartitionRuntimeSpatialHash;
};