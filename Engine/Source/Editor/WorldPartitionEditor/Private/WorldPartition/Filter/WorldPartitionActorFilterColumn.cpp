// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/Filter/WorldPartitionActorFilterColumn.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/SNullWidget.h"
#include "ScopedTransaction.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "WorldPartition/Filter/WorldPartitionActorFilterTreeItems.h"
#include "WorldPartition/Filter/WorldPartitionActorFilterMode.h"
#include "WorldPartition/Filter/WorldPartitionActorFilter.h"

#define LOCTEXT_NAMESPACE "WorldPartitionActorFilter"

FName FWorldPartitionActorFilterColumn::ID("WorldPartitionActorFilter");

FName FWorldPartitionActorFilterColumn::GetID()
{
	return ID;
}

SHeaderRow::FColumn::FArguments FWorldPartitionActorFilterColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnID())
		.FixedWidth(48.f)
		.HAlignHeader(HAlign_Center)
		.VAlignHeader(VAlign_Center)
		.HAlignCell(HAlign_Center)
		.VAlignCell(VAlign_Center)
		.DefaultTooltip(LOCTEXT("WorldPartitionActorFilterColumnToolTip", "Filter State"))
		[
			SNew(STextBlock)
			.Margin(FMargin(0.0f))
			.Text(LOCTEXT("WorldPartitionActorFilterColumnName", "Filter"))
		];
}

const TSharedRef<SWidget> FWorldPartitionActorFilterColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	TSharedPtr<ISceneOutliner> SceneOutliner = TreeItem->WeakSceneOutliner.Pin();
	const FWorldPartitionActorFilterMode* Mode = static_cast<const FWorldPartitionActorFilterMode*>(SceneOutliner->GetMode());

	if (FWorldPartitionActorFilterDataLayerItem* DataLayerItem = TreeItem->CastTo<FWorldPartitionActorFilterDataLayerItem>())
	{
		auto GetOverrideCheckedState = [DataLayerItem, Mode]() -> ECheckBoxState
		{
			if (const FWorldPartitionActorFilter* Filter = DataLayerItem->GetFilter())
			{
				const FWorldPartitionActorFilterMode::FFilter::FDataLayerFilters& DataLayerFilters = Mode->FindChecked(Filter);
				const FWorldPartitionActorFilterMode::FFilter::FDataLayerFilter& DataLayerFilter = DataLayerFilters.FindChecked(DataLayerItem->GetAssetPath());

				return DataLayerFilter.bOverride.IsSet() ? (DataLayerFilter.bOverride.GetValue() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked) : ECheckBoxState::Undetermined;
			}

			return ECheckBoxState::Undetermined;
		};

		auto OnOverrideCheckedStateChanged = [DataLayerItem, Mode](ECheckBoxState NewState)
		{
			if (const FWorldPartitionActorFilter* Filter = DataLayerItem->GetFilter())
			{
				FWorldPartitionActorFilterMode::FFilter::FDataLayerFilters& DataLayerFilters = Mode->FindChecked(Filter);
				FWorldPartitionActorFilterMode::FFilter::FDataLayerFilter& DataLayerFilter = DataLayerFilters.FindChecked(DataLayerItem->GetAssetPath());
				DataLayerFilter.bOverride = NewState == ECheckBoxState::Checked ? true : false;
				// If we are removing override set value to default
				if (!DataLayerFilter.bOverride.GetValue())
				{
					DataLayerFilter.bIncluded = DataLayerItem->GetDefaultValue();
				}
			}
		};

		auto GetCheckedState = [DataLayerItem, Mode]() -> ECheckBoxState
		{
			if (const FWorldPartitionActorFilter* Filter = DataLayerItem->GetFilter())
			{
				const FWorldPartitionActorFilterMode::FFilter::FDataLayerFilters& DataLayerFilters = Mode->FindChecked(Filter);
				const FWorldPartitionActorFilterMode::FFilter::FDataLayerFilter& DataLayerFilter = DataLayerFilters.FindChecked(DataLayerItem->GetAssetPath());

				if (!DataLayerFilter.bOverride.IsSet())
				{
					return ECheckBoxState::Undetermined;
				}

				return DataLayerFilter.bIncluded.IsSet() ? (DataLayerFilter.bIncluded.GetValue() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked) : ECheckBoxState::Undetermined;
			}

			return ECheckBoxState::Undetermined;
		};

		auto OnCheckedStateChanged = [DataLayerItem, Mode](ECheckBoxState NewState)
		{
			if (const FWorldPartitionActorFilter* Filter = DataLayerItem->GetFilter())
			{
				FWorldPartitionActorFilterMode::FFilter::FDataLayerFilters& DataLayerFilters = Mode->FindChecked(Filter);
				FWorldPartitionActorFilterMode::FFilter::FDataLayerFilter& DataLayerFilter = DataLayerFilters.FindChecked(DataLayerItem->GetAssetPath());
				DataLayerFilter.bIncluded = NewState == ECheckBoxState::Checked ? true : false;
			}
		};

		auto IsCheckedStateEnabled = [DataLayerItem, Mode]() -> bool
		{
			if (const FWorldPartitionActorFilter* Filter = DataLayerItem->GetFilter())
			{
				FWorldPartitionActorFilterMode::FFilter::FDataLayerFilters& DataLayerFilters = Mode->FindChecked(Filter);
				const FWorldPartitionActorFilterMode::FFilter::FDataLayerFilter& DataLayerFilter = DataLayerFilters.FindChecked(DataLayerItem->GetAssetPath());

				return DataLayerFilter.bOverride.IsSet() && DataLayerFilter.bOverride.GetValue();
			}
			return false;
		};

		return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
					.Padding(0, 0, 0, 0)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SCheckBox)
						.IsChecked_Lambda(GetOverrideCheckedState)
						.OnCheckStateChanged_Lambda(OnOverrideCheckedStateChanged)
						.ToolTipText(LOCTEXT("WorldPartitionActorFilterOverrideToolTip", "Override default"))
					]
				+SHorizontalBox::Slot()
					.Padding(0, 0, 0, 0)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SCheckBox)
						.IsChecked_Lambda(GetCheckedState)
						.OnCheckStateChanged_Lambda(OnCheckedStateChanged)
						.IsEnabled_Lambda(IsCheckedStateEnabled)
						.ToolTipText(LOCTEXT("WorldPartitionActorFilterToolTip", "Included/Excluded"))
					];
	}
	
	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE