// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/Filter/WorldPartitionActorFilterHierarchy.h"
#include "WorldPartition/Filter/WorldPartitionActorFilterMode.h"
#include "WorldPartition/Filter/WorldPartitionActorFilterTreeItems.h"

FWorldPartitionActorFilterHierarchy::FWorldPartitionActorFilterHierarchy(FWorldPartitionActorFilterMode* Mode)
	:ISceneOutlinerHierarchy(Mode)
{
	
}

FWorldPartitionActorFilterHierarchy::~FWorldPartitionActorFilterHierarchy()
{
	
}

TUniquePtr<FWorldPartitionActorFilterHierarchy> FWorldPartitionActorFilterHierarchy::Create(FWorldPartitionActorFilterMode* Mode)
{
	return TUniquePtr<FWorldPartitionActorFilterHierarchy>(new FWorldPartitionActorFilterHierarchy(Mode));
}

void FWorldPartitionActorFilterHierarchy::CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	FWorldPartitionActorFilterMode* FilterMode = static_cast<FWorldPartitionActorFilterMode*>(Mode);
	check(FilterMode);
	const FWorldPartitionActorFilter* Filter = FilterMode->GetFilter();
	check(Filter);
	if (FSceneOutlinerTreeItemPtr FilterTreeItem = Mode->CreateItemFor<FWorldPartitionActorFilterItem>(FWorldPartitionActorFilterItem::FTreeItemData(Filter), true))
	{
		OutItems.Add(FilterTreeItem);
		CreateChildren(FilterTreeItem, OutItems, true);
	}
}

void FWorldPartitionActorFilterHierarchy::CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	CreateChildren(Item, OutItems, false);
}

void FWorldPartitionActorFilterHierarchy::CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutItems, bool bRecursive) const
{
	if (FWorldPartitionActorFilterItem* FilterItem = Item->CastTo<FWorldPartitionActorFilterItem>())
	{
		for (auto& [AssetPath, DataLayerFilter] : FilterItem->GetFilter()->DataLayerFilters)
		{
			if (FSceneOutlinerTreeItemPtr FilterTreeDataLayerItem = Mode->CreateItemFor<FWorldPartitionActorFilterDataLayerItem>(FWorldPartitionActorFilterDataLayerItem::FTreeItemData(FilterItem->GetFilter(), AssetPath), true))
			{
				OutItems.Add(FilterTreeDataLayerItem);
			}
		}

		for (auto& [ActorGuid, ChildFilter] : FilterItem->GetFilter()->GetChildFilters())
		{
			if (FSceneOutlinerTreeItemPtr FilterTreeItem = Mode->CreateItemFor<FWorldPartitionActorFilterItem>(FWorldPartitionActorFilterItem::FTreeItemData(ChildFilter), true))
			{
				OutItems.Add(FilterTreeItem);
				if (bRecursive)
				{
					CreateChildren(FilterTreeItem, OutItems, true);
				}
			}
		}
	}
}

FSceneOutlinerTreeItemPtr FWorldPartitionActorFilterHierarchy::FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate /* = false */)
{
	const FWorldPartitionActorFilter* ParentFilter = nullptr;

	if (const FWorldPartitionActorFilterItem* FilterItem = Item.CastTo<FWorldPartitionActorFilterItem>())
	{
		ParentFilter = FilterItem->GetFilter()->GetParentFilter();
		
	}
	else if (const FWorldPartitionActorFilterDataLayerItem* DataLayerItem = Item.CastTo<FWorldPartitionActorFilterDataLayerItem>())
	{
		ParentFilter = DataLayerItem->GetFilter();
	}

	if (!ParentFilter)
	{
		return nullptr;
	}

	if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(FWorldPartitionActorFilterItem::ComputeTreeItemID(FWorldPartitionActorFilterItem::FTreeItemData(ParentFilter))))
	{
		return *ParentItem;
	}
	else if (bCreate)
	{
		return Mode->CreateItemFor<FWorldPartitionActorFilterItem>(FWorldPartitionActorFilterItem::FTreeItemData(ParentFilter));
	}
		
	return nullptr;
}
