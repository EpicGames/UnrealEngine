// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/Filter/WorldPartitionActorFilterMode.h"
#include "WorldPartition/Filter/WorldPartitionActorFilterHierarchy.h"
#include "WorldPartition/Filter/WorldPartitionActorFilterTreeItems.h"
#include "Algo/Transform.h"

#define LOCTEXT_NAMESPACE "WorldPartitionActorFilterMode"

FWorldPartitionActorFilterMode::FWorldPartitionActorFilterMode(SSceneOutliner* InSceneOutliner, TSharedPtr<FWorldPartitionActorFilterMode::FFilter> InFilter)
	: ISceneOutlinerMode(InSceneOutliner)
	, Filter(InFilter)
{
}

void FWorldPartitionActorFilterMode::Rebuild()
{
	Hierarchy = CreateHierarchy();
}

int32 FWorldPartitionActorFilterMode::GetTypeSortPriority(const ISceneOutlinerTreeItem& Item) const
{
	if (Item.IsA<FWorldPartitionActorFilterDataLayerItem>())
	{
		return 0;
	}
	
	return 1;
}

TUniquePtr<ISceneOutlinerHierarchy> FWorldPartitionActorFilterMode::CreateHierarchy()
{
	TUniquePtr<FWorldPartitionActorFilterHierarchy> NewHierarchy = FWorldPartitionActorFilterHierarchy::Create(this);
	return NewHierarchy;
}

FWorldPartitionActorFilterMode::FFilter::FFilter(TSharedPtr<FWorldPartitionActorFilter> InLevelFilter, const TArray<const FWorldPartitionActorFilter*>& InSelectedFilters)
	: LevelFilter(InLevelFilter)
{
	Initialize(LevelFilter.Get());
	for (int i = 0; i < InSelectedFilters.Num(); ++i)
	{
		if (i == 0)
		{
			Override(InSelectedFilters[i]);
		}
		else
		{
			Merge(InSelectedFilters[i]);
		}
	}
}

void FWorldPartitionActorFilterMode::FFilter::Initialize(const FWorldPartitionActorFilter* InFilter)
{
	TFunction<void(const FWorldPartitionActorFilter*)> InitializeInternal = [this, &InitializeInternal](const FWorldPartitionActorFilter* InFilterInternal)
	{
		FFilter::FDataLayerFilters& DataLayerFilters = FilterValues.Add(InFilterInternal);
		for (auto& [AssetPath, DataLayerFilter] : InFilterInternal->DataLayerFilters)
		{
			// Initialize new merged filter to not overriden and overriden value false
			DataLayerFilters.Add(AssetPath, FFilter::FDataLayerFilter(false, DataLayerFilter.bIncluded));
		}

		for (auto& [ActorGuid, WorldPartitionActorFilter] : InFilterInternal->GetChildFilters())
		{
			InitializeInternal(WorldPartitionActorFilter);
		}
	};
	InitializeInternal(InFilter);
}

void FWorldPartitionActorFilterMode::FFilter::Override(const FWorldPartitionActorFilter* Other)
{
	TFunction<void(const FWorldPartitionActorFilter*, const FWorldPartitionActorFilter*)> OverrideInternal = [this, &OverrideInternal](const FWorldPartitionActorFilter* InValue, const FWorldPartitionActorFilter* InOverride)
	{
		if (FFilter::FDataLayerFilters* DataLayerFilters = FilterValues.Find(InValue))
		{
			for (auto& [AssetPath, DataLayerFilter] : InOverride->DataLayerFilters)
			{
				if (FFilter::FDataLayerFilter* Found = DataLayerFilters->Find(AssetPath))
				{
					Found->bOverride = true;
					Found->bIncluded = DataLayerFilter.bIncluded;
				}
			}
		}

		for (auto& [ActorGuid, WorldPartitionActorFilter] : InOverride->GetChildFilters())
		{
			if (FWorldPartitionActorFilter*const* ChildFilter = InValue->GetChildFilters().Find(ActorGuid))
			{
				OverrideInternal(*ChildFilter, WorldPartitionActorFilter);
			}
		}
	};
	OverrideInternal(LevelFilter.Get(), Other);
}

void FWorldPartitionActorFilterMode::FFilter::Merge(const FWorldPartitionActorFilter* Other)
{
	TFunction<void(const FWorldPartitionActorFilter*, const FWorldPartitionActorFilter*)> MergeInternal = [this, &MergeInternal](const FWorldPartitionActorFilter* InValue, const FWorldPartitionActorFilter* InMerge)
	{
		if (FFilter::FDataLayerFilters* DataLayerFilters = FilterValues.Find(InValue))
		{
			for (auto& [AssetPath, DataLayerFilter] : *DataLayerFilters)
			{
				const FWorldPartitionActorFilter::FDataLayerFilter* Found = InMerge ? InMerge->DataLayerFilters.Find(AssetPath) : nullptr;
	
				if (DataLayerFilter.bOverride.IsSet())
				{
					if (DataLayerFilter.bOverride.GetValue() != !!Found)
					{
						DataLayerFilter.bOverride.Reset();
						DataLayerFilter.bIncluded.Reset();
					}
					else if (DataLayerFilter.bOverride.GetValue() && DataLayerFilter.bIncluded.IsSet() && Found && DataLayerFilter.bIncluded.GetValue() != Found->bIncluded)
					{
						DataLayerFilter.bIncluded.Reset();
					}
				}
			}
		}

		for (auto& [ActorGuid, WorldPartitionActorFilter] : InValue->GetChildFilters())
		{
			FWorldPartitionActorFilter*const* MergeChild = InMerge ? InMerge->GetChildFilters().Find(ActorGuid) : nullptr;
			MergeInternal(WorldPartitionActorFilter, MergeChild ? *MergeChild : nullptr );
		}
	};
	MergeInternal(LevelFilter.Get(), Other);
}

void FWorldPartitionActorFilterMode::Apply(FWorldPartitionActorFilter& InOutResult) const
{
	Filter->Apply(&InOutResult);
}

void FWorldPartitionActorFilterMode::FFilter::Apply(FWorldPartitionActorFilter* InOutResult) const
{
	TFunction<bool(const FWorldPartitionActorFilter*, FWorldPartitionActorFilter*)> ApplyInternal = [this, &ApplyInternal](const FWorldPartitionActorFilter* InValue, FWorldPartitionActorFilter* InResult)
	{
		const FFilter::FDataLayerFilters& MergedDataLayerFilter = FilterValues.FindChecked(InValue);
		for (auto& [AssetPath, DataLayerFilter] : InValue->DataLayerFilters)
		{
			const auto& Found = MergedDataLayerFilter.FindChecked(AssetPath);
			if (Found.bOverride.IsSet())
			{
				if (Found.bOverride.GetValue())
				{
					if (Found.bIncluded.IsSet())
					{
						InResult->DataLayerFilters.FindOrAdd(AssetPath).bIncluded = Found.bIncluded.GetValue();
					}
				}
				else
				{
					InResult->DataLayerFilters.Remove(AssetPath);
				}
			}
		}

		for (auto& [ActorGuid, WorldPartitionActorFilter] : InValue->GetChildFilters())
		{
			if (FWorldPartitionActorFilter* const* ChildResultFilter = InResult->GetChildFilters().Find(ActorGuid))
			{
				if (!ApplyInternal(WorldPartitionActorFilter, *ChildResultFilter))
				{
					InResult->RemoveChildFilter(ActorGuid);
				}
			}
			else
			{
				FWorldPartitionActorFilter ChildResult;
				if (ApplyInternal(WorldPartitionActorFilter, &ChildResult))
				{
					InResult->AddChildFilter(ActorGuid, new FWorldPartitionActorFilter(MoveTemp(ChildResult)));
				}
			}
		}

		return InResult->DataLayerFilters.Num() > 0 || InResult->GetChildFilters().Num();
	};

	ApplyInternal(LevelFilter.Get(), InOutResult);
}

#undef LOCTEXT_NAMESPACE