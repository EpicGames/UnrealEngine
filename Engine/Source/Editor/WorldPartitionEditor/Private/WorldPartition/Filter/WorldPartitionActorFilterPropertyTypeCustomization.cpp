// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/Filter/WorldPartitionActorFilterPropertyTypeCustomization.h"

#include "IPropertyUtilities.h"
#include "IPropertyTypeCustomization.h"
#include "Containers/Array.h"
#include "CoreTypes.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SButton.h"
#include "Internationalization/Internationalization.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "WorldPartition/Filter/WorldPartitionActorFilter.h"
#include "LevelInstance/LevelInstanceComponent.h"
#include "Modules/ModuleManager.h"
#include "ISceneOutlinerMode.h"
#include "ISceneOutliner.h"
#include "SceneOutlinerModule.h"
#include "WorldPartition/Filter/WorldPartitionActorFilterMode.h"
#include "WorldPartition/Filter/WorldPartitionActorFilterColumn.h"
#include "Settings/EditorExperimentalSettings.h"

#define LOCTEXT_NAMESPACE "WorldPartitionActorFilter"

FWorldPartitionActorFilterPropertyTypeCustomization::~FWorldPartitionActorFilterPropertyTypeCustomization()
{
	FWorldPartitionActorFilter::GetOnWorldPartitionActorFilterChanged().Remove(WorldPartitionActorFilterChangedHandle);
}

void FWorldPartitionActorFilterPropertyTypeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	if (!GetDefault<UEditorExperimentalSettings>()->bEnableWorldPartitionActorFilters)
	{
		StructPropertyHandle->MarkHiddenByCustomization();
		return;
	}

	TArray<UObject*> OuterObjects;
	StructPropertyHandle->GetOuterObjects(OuterObjects);
		
	// Create Mode Filter which holds the final values for the filter
	TSharedPtr<FWorldPartitionActorFilterMode::FFilter> ModeFilter = CreateModeFilter(OuterObjects);
	if (!ModeFilter)
	{
		return;
	}
	
	FCreateSceneOutlinerMode ModeFactory = FCreateSceneOutlinerMode::CreateLambda([=](SSceneOutliner* Outliner)
	{
		return new FWorldPartitionActorFilterMode(Outliner, ModeFilter);
	});

	FSceneOutlinerInitializationOptions InitOptions;
	{
		InitOptions.bShowHeaderRow = false;
		InitOptions.bFocusSearchBoxWhenOpened = false;
		InitOptions.bShowCreateNewFolder = false;
		InitOptions.ModeFactory = ModeFactory;
	}

	InitOptions.ColumnMap.Add(FWorldPartitionActorFilterColumn::ID, FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0, FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner) { return MakeShareable(new FWorldPartitionActorFilterColumn(InSceneOutliner)); }), false, TOptional<float>(), LOCTEXT("WorldPartitionActorFilterColumn","Filter")));
	InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 1, FCreateSceneOutlinerColumn(), false, TOptional<float>(), FSceneOutlinerBuiltInColumnTypes::Label_Localized()));

	// Actor selector to allow the user to choose a parent actor
	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");

	// When value changes refresh details + filter actors
	const FSimpleDelegate OnValueChanged = FSimpleDelegate::CreateLambda([PropertyUtils = StructCustomizationUtils.GetPropertyUtilities()]()
	{
		PropertyUtils->ForceRefresh();
		FWorldPartitionActorFilter::RequestFilterRefresh(true);
	});

	// Listen to outside changes to Filters to refresh details panel (Undo/Redo)
	WorldPartitionActorFilterChangedHandle = FWorldPartitionActorFilter::GetOnWorldPartitionActorFilterChanged().AddLambda([PropertyUtils = StructCustomizationUtils.GetPropertyUtilities()]()
	{
		PropertyUtils->ForceRefresh();
	});

	SceneOutliner = SceneOutlinerModule.CreateSceneOutliner(InitOptions);

	auto OnApplyFilter = [StructPropertyHandle, OnValueChanged, this]()
	{
		const FWorldPartitionActorFilterMode* Mode = static_cast<const FWorldPartitionActorFilterMode*>(SceneOutliner->GetMode());
		ApplyFilter(StructPropertyHandle, *Mode);
		OnValueChanged.Execute();
		return FReply::Handled();
	};

	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SBox)
		.MinDesiredWidth(2000.f) // Set very large as it is the only way for the Outliner to expand to take all the space
		.MinDesiredHeight(150.f)
		.MaxDesiredHeight(300.f)
		.IsEnabled(StructPropertyHandle->IsEditable())
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SceneOutliner.ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 4.0f, 0.0f, 4.0f)
			[
				SNew(SButton)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.OnClicked_Lambda(OnApplyFilter)
				.Text(LOCTEXT("ApplyFilter", "Apply"))
				.ToolTipText(LOCTEXT("ApplyFilterToolTip", "Apply Filter"))
			]
		]
	];
		
	// This gets called when hitting ResetToDefault
	StructPropertyHandle->SetOnPropertyValueChanged(OnValueChanged);
}

#undef LOCTEXT_NAMESPACE