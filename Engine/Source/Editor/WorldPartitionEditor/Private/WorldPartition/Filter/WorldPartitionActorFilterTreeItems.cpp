// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/Filter/WorldPartitionActorFilterTreeItems.h"
#include "WorldPartition/Filter/WorldPartitionActorFilter.h"
#include "LevelInstance/LevelInstanceActor.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "ISceneOutliner.h"
#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerStandaloneTypes.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "WorldPartitionActorFilter"

const FSceneOutlinerTreeItemType FWorldPartitionActorFilterItemBase::Type(&ISceneOutlinerTreeItem::Type);
const FSceneOutlinerTreeItemType FWorldPartitionActorFilterItem::Type(&FWorldPartitionActorFilterItemBase::Type);
const FSceneOutlinerTreeItemType FWorldPartitionActorFilterDataLayerItem::Type(&FWorldPartitionActorFilterItemBase::Type);

struct SWorldPartitionActorFilterLabel : FSceneOutlinerCommonLabelData, public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SWorldPartitionActorFilterLabel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, ISceneOutlinerTreeItem& TreeItem, ISceneOutliner& SceneOutliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
	{
		TreeItemPtr = StaticCastSharedRef<ISceneOutlinerTreeItem>(TreeItem.AsShared());
		WeakSceneOutliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared());
				
		HighlightText = SceneOutliner.GetFilterHighlightText();
								
		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FSceneOutlinerDefaultTreeItemMetrics::IconPadding())
			[
				SNew(SBox)
				.WidthOverride(FSceneOutlinerDefaultTreeItemMetrics::IconSize())
				.HeightOverride(FSceneOutlinerDefaultTreeItemMetrics::IconSize())
				[
					SNew(SImage)
					.Image(this, &SWorldPartitionActorFilterLabel::GetIcon)
				]
			]
			+SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			.Padding(0.0f, 2.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(STextBlock)
					.Text(this, &SWorldPartitionActorFilterLabel::GetDisplayText)
					.ToolTipText(this, &SWorldPartitionActorFilterLabel::GetDisplayText)
					.ColorAndOpacity(this, &SWorldPartitionActorFilterLabel::GetForegroundColor)
				]
			]
		];
	}

private:
	TWeakPtr<ISceneOutlinerTreeItem> TreeItemPtr;
	TAttribute<FText> HighlightText;


	FText GetDisplayText() const
	{
		auto Item = TreeItemPtr.Pin();
		return Item.IsValid() ? FText::FromString(Item->GetDisplayString()) : FText();
	}
	
	const FSlateBrush* GetIcon() const
	{
		if (FSceneOutlinerTreeItemPtr PinnedPtr = TreeItemPtr.Pin(); PinnedPtr.IsValid())
		{
			if (WeakSceneOutliner.IsValid())
			{
				if (FWorldPartitionActorFilterItemBase* WorldPartitionActorFilterItem = PinnedPtr->CastTo<FWorldPartitionActorFilterItemBase>())
				{
					if (UClass* BrushClass = WorldPartitionActorFilterItem->GetIconClass())
					{
						const FSlateBrush* CachedBrush = WeakSceneOutliner.Pin()->GetCachedIconForClass(BrushClass->GetFName());
						if (CachedBrush != nullptr)
						{
							return CachedBrush;
						}
						else
						{
							const FSlateBrush* FoundSlateBrush = FSlateIconFinder::FindIconBrushForClass(BrushClass);
							WeakSceneOutliner.Pin()->CacheIconForClass(BrushClass->GetFName(), const_cast<FSlateBrush*>(FoundSlateBrush));
							return FoundSlateBrush;
						}
					}
				}
			}
		}
		
		return nullptr;
	}
		
	FSlateColor GetForegroundColor() const
	{
		if (auto BaseColor = FSceneOutlinerCommonLabelData::GetForegroundColor(*TreeItemPtr.Pin()))
		{
			return BaseColor.GetValue();
		}

		return FSlateColor::UseForeground();
	}
};

TSharedRef<SWidget> FWorldPartitionActorFilterItemBase::GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	return SNew(SWorldPartitionActorFilterLabel, *this, Outliner, InRow);
}

FWorldPartitionActorFilterItem::FWorldPartitionActorFilterItem(const FWorldPartitionActorFilterItem::FTreeItemData& InData)
	: FWorldPartitionActorFilterItemBase(FWorldPartitionActorFilterItem::Type)
	, Data(InData)
{
}

FSceneOutlinerTreeItemID FWorldPartitionActorFilterItem::GetID() const
{
	return ComputeTreeItemID(Data);
}

FString FWorldPartitionActorFilterItem::GetDisplayString() const
{
	return Data.Filter->DisplayName;
}

UClass* FWorldPartitionActorFilterItem::GetIconClass() const
{
	return ALevelInstance::StaticClass();
}

FSceneOutlinerTreeItemID FWorldPartitionActorFilterItem::ComputeTreeItemID(const FWorldPartitionActorFilterItem::FTreeItemData& InData)
{
	return FSceneOutlinerTreeItemID(GetTypeHash(InData.Filter));
}

FWorldPartitionActorFilterDataLayerItem::FWorldPartitionActorFilterDataLayerItem(const FWorldPartitionActorFilterDataLayerItem::FTreeItemData& InData)
	: FWorldPartitionActorFilterItemBase(FWorldPartitionActorFilterDataLayerItem::Type)
	, Data(InData)
{
}

FSceneOutlinerTreeItemID FWorldPartitionActorFilterDataLayerItem::GetID() const
{
	return ComputeTreeItemID(Data);
}

FString FWorldPartitionActorFilterDataLayerItem::GetDisplayString() const
{
	return Data.Filter->DataLayerFilters[Data.AssetPath].DisplayName;
}

UClass* FWorldPartitionActorFilterDataLayerItem::GetIconClass() const
{
	return UDataLayerAsset::StaticClass();
}

FSceneOutlinerTreeItemID FWorldPartitionActorFilterDataLayerItem::ComputeTreeItemID(const FWorldPartitionActorFilterDataLayerItem::FTreeItemData& InData)
{
	return FSceneOutlinerTreeItemID(GetTypeHash(InData.Filter) ^ GetTypeHash(InData.AssetPath));
}

bool FWorldPartitionActorFilterDataLayerItem::GetDefaultValue() const
{
	check(IsValid());
	return Data.Filter->DataLayerFilters.FindChecked(Data.AssetPath).bIncluded;
}

#undef LOCTEXT_NAMESPACE