// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODCompareCommands.h"

#define LOCTEXT_NAMESPACE "HLODCompareCommands"

void FHLODCompareCommands::RegisterCommands()
{
	UI_COMMAND(ViewModeLit, "Lit", "Switch to Lit view mode", EUserInterfaceActionType::RadioButton, FInputChord(EModifierKey::Alt, EKeys::One));
	UI_COMMAND(ViewModeWireframe, "Wireframe", "Switch to Wireframe view mode", EUserInterfaceActionType::RadioButton, FInputChord(EModifierKey::Alt, EKeys::Two));
	UI_COMMAND(ViewModeBaseColor, "Base Color", "Switch to Base Color buffer visualization", EUserInterfaceActionType::RadioButton, FInputChord(EModifierKey::Alt, EKeys::Three));
	UI_COMMAND(ViewModeMetallic, "Metallic", "Switch to Metallic buffer visualization", EUserInterfaceActionType::RadioButton, FInputChord(EModifierKey::Alt, EKeys::Four));
	UI_COMMAND(ViewModeRoughness, "Roughness", "Switch to Roughness buffer visualization", EUserInterfaceActionType::RadioButton, FInputChord(EModifierKey::Alt, EKeys::Five));
	UI_COMMAND(ViewModeSpecular, "Specular", "Switch to Specular buffer visualization", EUserInterfaceActionType::RadioButton, FInputChord(EModifierKey::Alt, EKeys::Six));
	UI_COMMAND(ViewModeWorldNormal, "World Normal", "Switch to World Normal buffer visualization", EUserInterfaceActionType::RadioButton, FInputChord(EModifierKey::Alt, EKeys::Seven));
	UI_COMMAND(GoToHLODDistance, "Go to HLOD Distance", "Move camera to the HLOD minimum visible distance", EUserInterfaceActionType::Button, FInputChord(EKeys::SpaceBar));
}

#undef LOCTEXT_NAMESPACE
