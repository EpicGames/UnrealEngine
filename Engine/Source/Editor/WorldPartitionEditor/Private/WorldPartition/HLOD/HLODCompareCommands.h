// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandInfo.h"
#include "Styling/AppStyle.h"

class FHLODCompareCommands : public TCommands<FHLODCompareCommands>
{
public:
	FHLODCompareCommands()
		: TCommands<FHLODCompareCommands>(TEXT("HLODCompare"), NSLOCTEXT("Contexts", "HLODCompare", "HLOD Compare"), NAME_None, FAppStyle::GetAppStyleSetName())
	{
	}

	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> ViewModeLit;
	TSharedPtr<FUICommandInfo> ViewModeWireframe;
	TSharedPtr<FUICommandInfo> ViewModeBaseColor;
	TSharedPtr<FUICommandInfo> ViewModeMetallic;
	TSharedPtr<FUICommandInfo> ViewModeRoughness;
	TSharedPtr<FUICommandInfo> ViewModeSpecular;
	TSharedPtr<FUICommandInfo> ViewModeWorldNormal;
	TSharedPtr<FUICommandInfo> GoToHLODDistance;
};
