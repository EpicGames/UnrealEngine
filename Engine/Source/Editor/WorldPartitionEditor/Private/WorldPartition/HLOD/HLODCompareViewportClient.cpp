// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODCompareViewportClient.h"
#include "WorldPartition/HLOD/SHLODCompareWindow.h"
#include "PreviewScene.h"

FHLODCompareViewportClient::FHLODCompareViewportClient(FPreviewScene* InPreviewScene, const TSharedRef<SEditorViewport>& InEditorViewportWidget, TWeakPtr<SHLODCompareWindow> InCompareWindow)
	: FEditorViewportClient(nullptr, InPreviewScene, InEditorViewportWidget)
	, CompareWindow(InCompareWindow)
{
	SetRealtime(true);
	SetViewportType(LVT_Perspective);
}

void FHLODCompareViewportClient::PerspectiveCameraMoved()
{
	FEditorViewportClient::PerspectiveCameraMoved();

	if (!bIsSyncing)
	{
		if (TSharedPtr<SHLODCompareWindow> Window = CompareWindow.Pin())
		{
			Window->OnCameraMoved(this);
		}
	}
}

void FHLODCompareViewportClient::SetViewMode(EViewModeIndex InViewModeIndex)
{
	FEditorViewportClient::SetViewMode(InViewModeIndex);

	if (!bIsSyncing)
	{
		if (TSharedPtr<SHLODCompareWindow> Window = CompareWindow.Pin())
		{
			Window->OnViewModeChanged(this);
		}
	}
}

void FHLODCompareViewportClient::SyncCameraFrom(const FHLODCompareViewportClient& Other)
{
	bIsSyncing = true;

	// Match the orbit camera state first. Alt+LMB orbit toggles the source viewport into
	// orbit mode, where stored ViewLocation/ViewRotation are in orbit space (composed via
	// ComputeOrbitMatrix to produce the actual view). Without matching the mode, the
	// sibling viewport interprets those orbit-space values as world space and the two
	// views drift apart.
	ToggleOrbitCamera(Other.bUsingOrbitCamera);

	SetViewLocation(Other.GetViewLocation());
	SetViewRotation(Other.GetViewRotation());
	SetLookAtLocation(Other.GetLookAtLocation());
	SetOrthoZoom(Other.GetOrthoZoom());
	bIsSyncing = false;
}

void FHLODCompareViewportClient::Draw(FViewport* InViewport, FCanvas* Canvas)
{
	// Pull buffer visualization mode from sibling right before drawing so it's always in sync
	if (TSharedPtr<FHLODCompareViewportClient> Sibling = SiblingClient.Pin())
	{
		CurrentBufferVisualizationMode = Sibling->CurrentBufferVisualizationMode;
	}

	// Disable temporal AA in buffer visualization modes to avoid flickering
	EngineShowFlags.SetTemporalAA(GetViewMode() != VMI_VisualizeBuffer);

	// Tick the preview scene's sky light and reflection captures
	if (FPreviewScene* Scene = GetPreviewScene())
	{
		Scene->UpdateCaptureContents();
	}

	FEditorViewportClient::Draw(InViewport, Canvas);
}

void FHLODCompareViewportClient::SyncViewModeFrom(const FHLODCompareViewportClient& Other)
{
	bIsSyncing = true;
	SetViewMode(Other.GetViewMode());
	bIsSyncing = false;
}
