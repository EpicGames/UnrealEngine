// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorViewportClient.h"

class FPreviewScene;
class SHLODCompareWindow;
class SEditorViewport;

class FHLODCompareViewportClient : public FEditorViewportClient
{
public:
	FHLODCompareViewportClient(FPreviewScene* InPreviewScene, const TSharedRef<SEditorViewport>& InEditorViewportWidget, TWeakPtr<SHLODCompareWindow> InCompareWindow);

	// FEditorViewportClient interface
	virtual void PerspectiveCameraMoved() override;
	virtual void SetViewMode(EViewModeIndex InViewModeIndex) override;
	virtual void Draw(FViewport* InViewport, FCanvas* Canvas) override;

	void SyncCameraFrom(const FHLODCompareViewportClient& Other);
	void SyncViewModeFrom(const FHLODCompareViewportClient& Other);

	/** Set a sibling viewport client to sync buffer visualization mode from before each draw. */
	void SetSiblingClient(TWeakPtr<FHLODCompareViewportClient> InSibling) { SiblingClient = InSibling; }

private:
	TWeakPtr<SHLODCompareWindow> CompareWindow;
	TWeakPtr<FHLODCompareViewportClient> SiblingClient;
	bool bIsSyncing = false;
};
