// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODEditorData.h"

#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODActorDesc.h"
#include "WorldPartition/HLOD/HLODLoaderAdapter.h"

#include "WorldPartition/ActorDescContainerInstanceCollection.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterActorList.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/WorldPartitionHelpers.h"

FWorldPartitionHLODEditorData::FWorldPartitionHLODEditorData(UWorldPartition* InWorldPartition)
	: WorldPartition(InWorldPartition)
	, LastStateUpdate(INDEX_NONE)
{
	WorldPartition->OnActorDescContainerInstanceRegistered.AddRaw(this, &FWorldPartitionHLODEditorData::OnActorDescContainerInstanceRegistered);
	WorldPartition->OnActorDescContainerInstanceUnregistered.AddRaw(this, &FWorldPartitionHLODEditorData::OnActorDescContainerInstanceUnregistered);

	// Since this is created upon WP init, we missed the first broadcasts for existing container instances. Register them manually.
	WorldPartition->ForEachActorDescContainerInstance([this](UActorDescContainerInstance* InContainerInstance)
	{
		OnActorDescContainerInstanceRegistered(InContainerInstance);
	});	

	ExternalDirtyActorsTracker = MakeUnique<FExternalDirtyActorsTracker>(WorldPartition->GetWorld()->PersistentLevel, this);
}

FWorldPartitionHLODEditorData::~FWorldPartitionHLODEditorData()
{
	WorldPartition->OnActorDescContainerInstanceRegistered.RemoveAll(this);
	WorldPartition->OnActorDescContainerInstanceUnregistered.RemoveAll(this);
}

void FWorldPartitionHLODEditorData::OnActorDescContainerInstanceRegistered(UActorDescContainerInstance* InContainerInstance)
{
	check(!PerContainerInstanceHLODActorDataMap.Contains(InContainerInstance));

	FContainerInstanceHLODActorData* ContainerInstanceHLODActorData = nullptr;

	// First pass, build mapping of GUID to HLOD scene node
	for (UActorDescContainerInstance::TConstIterator<AWorldPartitionHLOD> HLODIterator(InContainerInstance); HLODIterator; ++HLODIterator)
	{
		// Only create an entry in the map if there are HLOD actors in this container instance.
		if (!ContainerInstanceHLODActorData)
		{
			ContainerInstanceHLODActorData = &PerContainerInstanceHLODActorDataMap.Emplace(InContainerInstance);
		} 

		const FHLODActorDesc& HLODActorDesc = *(FHLODActorDesc*)HLODIterator->GetActorDesc();
		const FGuid& HLODActorGuid = HLODActorDesc.GetGuid();

		TUniquePtr<FHLODSceneNode>& HLODSceneNode = ContainerInstanceHLODActorData->HLODActorNodes.Emplace(HLODActorGuid, new FHLODSceneNode());
		HLODSceneNode->Bounds = HLODActorDesc.GetEditorBounds().TransformBy(WorldPartition->GetInstanceTransform());
	}

	// If there are no HLOD actors in this container instance, early out
	if (!ContainerInstanceHLODActorData)
	{
		return;
	}

	// Second pass, build a hierarchy now that nodes were all created
	for (UActorDescContainerInstance::TConstIterator<AWorldPartitionHLOD> HLODIterator(InContainerInstance); HLODIterator; ++HLODIterator)
	{
		const FHLODActorDesc& HLODActorDesc = *(FHLODActorDesc*)HLODIterator->GetActorDesc();
		const FGuid& HLODActorGuid = HLODActorDesc.GetGuid();
		TUniquePtr<FHLODSceneNode>& HLODSceneNode = ContainerInstanceHLODActorData->HLODActorNodes.FindChecked(HLODActorGuid);

		for (const FGuid& ChildHLODActorGuid : HLODActorDesc.GetChildHLODActors())
		{
			if (TUniquePtr<FHLODSceneNode>* ChildHLODSceneNodePtr = ContainerInstanceHLODActorData->HLODActorNodes.Find(ChildHLODActorGuid))
			{
				FHLODSceneNode* ChildHLODSceneNode = ChildHLODSceneNodePtr->Get();
				ChildHLODSceneNode->ParentHLOD = HLODSceneNode.Get();

				HLODSceneNode->ChildrenHLODs.Add(ChildHLODSceneNode);
			}
		}
	}

	// Cache top level HLOD nodes for a faster iteration during the subsystem tick.
	for (const auto& [HLODActorGuid, HLODActorNode] : ContainerInstanceHLODActorData->HLODActorNodes)
	{
		if (HLODActorNode->ParentHLOD == nullptr)
		{
			ContainerInstanceHLODActorData->TopLevelHLODActorNodes.Add(HLODActorNode.Get());
		}
	}
}

void FWorldPartitionHLODEditorData::OnActorDescContainerInstanceUnregistered(UActorDescContainerInstance* InContainerInstance)
{
	PerContainerInstanceHLODActorDataMap.Remove(InContainerInstance);
}

struct FBoundsWithVolume
{
	FBoundsWithVolume(const FBox& InBox)
		: Box(InBox)
		, Volume(Box.GetVolume())
	{
	}

	FBox Box;
	FBox::FReal Volume;
};

// Gather Pinned Actors bounds - also include references & contained actors if the pinned actor is a container.
static void GatherLoadedActorsBounds(TArray<FBoundsWithVolume>& OutLoadedBounds, TSet<const FWorldPartitionActorDescInstance*>& InProcessedActors, const FWorldPartitionActorDescInstance* InActorDescInstance, const UActorDescContainerInstance* InContainerInstance, const TOptional<FTransform>& InContainerTransform = TOptional<FTransform>())
{
	if (InActorDescInstance)
	{
		bool bIsAlreadyInSet;
		InProcessedActors.FindOrAdd(InActorDescInstance, &bIsAlreadyInSet);

		if (bIsAlreadyInSet)
		{
			return;
		}

		// The actor itself - Include the actor bounds only if HLOD relevant
		if (InActorDescInstance->GetIsSpatiallyLoaded() && InActorDescInstance->GetActorIsHLODRelevant() && InActorDescInstance->IsEditorRelevant())
		{
			const FBox Box = InContainerTransform.IsSet() ? InActorDescInstance->GetEditorBounds().TransformBy(InContainerTransform.GetValue()) : InActorDescInstance->GetEditorBounds();
			OutLoadedBounds.Emplace(Box);
		}

		// Test its references
		for (const FGuid& ReferenceGuid : InActorDescInstance->GetReferences())
		{
			if (const FWorldPartitionActorDescInstance* ReferenceActorDescInstance = InContainerInstance->GetActorDescInstance(ReferenceGuid))
			{
				GatherLoadedActorsBounds(OutLoadedBounds, InProcessedActors, ReferenceActorDescInstance, InContainerInstance, InContainerTransform);
			}
		}

		// If it's a container, test the contained actors
		if (InActorDescInstance->IsChildContainerInstance())
		{
			FWorldPartitionActorDesc::FContainerInstance ContainerInstance;
			if (InActorDescInstance->GetChildContainerInstance(ContainerInstance))
			{
				FTransform ContainerWorldSpaceTransform = InContainerTransform.IsSet() ? ContainerInstance.Transform * InContainerTransform.GetValue() : ContainerInstance.Transform;
				for (UActorDescContainerInstance::TConstIterator<> Iterator(ContainerInstance.ContainerInstance); Iterator; ++Iterator)
				{
					GatherLoadedActorsBounds(OutLoadedBounds, InProcessedActors, *Iterator, ContainerInstance.ContainerInstance, ContainerWorldSpaceTransform);
				}
			}
		}
	}
}

bool FWorldPartitionHLODEditorData::IsLoadedActorsStateInitialized() const
{
	return LastStateUpdate != INDEX_NONE;
}

void FWorldPartitionHLODEditorData::ClearLoadedActorsState()
{
	// Increment update counter - used to quickly find out if an HLOD actor needs be hidden without having to flag the whole hierarchy
	LastStateUpdate++;
}

void FWorldPartitionHLODEditorData::UpdateLoadedActorsState()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWorldPartitionHLODEditorData::UpdateLoadedActorsState);

	// Increment update counter - used to quickly find out if an HLOD actor needs be hidden without having to flag the whole hierarchy
	LastStateUpdate++;
	
	TArray<FBoundsWithVolume> LoadedBounds;

	// Gather LoaderAdapter
	for (UWorldPartitionEditorLoaderAdapter* EditorLoaderAdapter : WorldPartition->GetRegisteredEditorLoaderAdapters())
	{
		if (IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = EditorLoaderAdapter->GetLoaderAdapter())
		{
			if (LoaderAdapter->IsLoaded() && LoaderAdapter->GetBoundingBox().IsSet())
			{
				LoadedBounds.Emplace(LoaderAdapter->GetBoundingBox().GetValue());
			}
		}
	}

	// Gather ActorLoaderInterface
	FWorldPartitionHelpers::ForEachActorDescInstance(WorldPartition, [&LoadedBounds](const FWorldPartitionActorDescInstance* ActorDescInstance)
	{
		if (ActorDescInstance->GetActorNativeClass()->ImplementsInterface(UWorldPartitionActorLoaderInterface::StaticClass()))
		{
			if (AActor* Actor = ActorDescInstance->GetActor(false))
			{
				if (IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = Cast<IWorldPartitionActorLoaderInterface>(Actor)->GetLoaderAdapter())
				{
					if (LoaderAdapter->IsLoaded() && LoaderAdapter->GetBoundingBox().IsSet())
					{
						LoadedBounds.Emplace(ActorDescInstance->GetEditorBounds());
					}
				}
			}
		}

		return true;
	});

	TSet<const FWorldPartitionActorDescInstance*> ProcessedActors;

	// Gather Pinned Actors bounds
	if (WorldPartition->PinnedActors)
	{
		for (const FWorldPartitionHandle& PinnedActor : WorldPartition->PinnedActors->GetActors())
		{
			if (PinnedActor.IsValid())
			{
				GatherLoadedActorsBounds(LoadedBounds, ProcessedActors, *PinnedActor, WorldPartition->GetActorDescContainerInstance());
			}
		}
	}
	
	// Gather Dirty Actors bounds
	for (FExternalDirtyActorsTracker::MapType::TConstIterator MapIt(ExternalDirtyActorsTracker->GetDirtyActors()); MapIt; ++MapIt)
	{
		const TWeakObjectPtr<AActor>& Actor = MapIt.Key();
		if (Actor.IsValid())
		{
			FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(Actor->GetActorGuid());
			GatherLoadedActorsBounds(LoadedBounds, ProcessedActors, ActorDescInstance, WorldPartition->GetActorDescContainerInstance());
		}
	}

	// Sort Bounds by volume
	Algo::Sort(LoadedBounds, [](const FBoundsWithVolume& BoxA, const FBoundsWithVolume& BoxB) { return BoxA.Volume > BoxB.Volume; });

	const auto UpdateNodeState = [StateUpdate = LastStateUpdate, &LoadedBounds](FHLODSceneNode* HLODSceneNode)
	{
		auto UpdateNodeStateImpl = [StateUpdate, &LoadedBounds](FHLODSceneNode* HLODSceneNode, auto& UpdateNodeStateRef) -> void
		{
			FBox HLODSceneNodeBox = HLODSceneNode->Bounds.GetBox();

			// Perform an overlap test that exclude shared edges. For example, this prevents neighbor landscape HLODs actors that are
			// perfectly matching landscape proxy bounds from being hidden when a landscape proxy is pinned.
			const bool bHasAnyIntersectingLoadedRegion = Algo::AnyOf(LoadedBounds, [&HLODSceneNodeBox](const FBoundsWithVolume& Bounds) 
			{
				const FBox BoxOverlap = Bounds.Box.Overlap(HLODSceneNodeBox);
				if (!BoxOverlap.IsValid)
				{
					return false;
				}

				const FVector OverlapSize = BoxOverlap.GetSize();
				return OverlapSize.X > UE_THRESH_POINTS_ARE_NEAR && OverlapSize.Y > UE_THRESH_POINTS_ARE_NEAR;
			});

			if (bHasAnyIntersectingLoadedRegion)
			{
				HLODSceneNode->HasIntersectingLoadedRegion = StateUpdate;

				for (FHLODSceneNode* Child : HLODSceneNode->ChildrenHLODs)
				{
					UpdateNodeStateRef(Child, UpdateNodeStateRef);
				}
			}
		};

		return UpdateNodeStateImpl(HLODSceneNode, UpdateNodeStateImpl);
	};

	// Update Nodes, starting from the top level HLODs down to their children
	for (auto& MapEntry : PerContainerInstanceHLODActorDataMap)
	{
		FContainerInstanceHLODActorData& ContainerInstanceHLODActorData = MapEntry.Value;

		for (FHLODSceneNode* HLODSceneNode : ContainerInstanceHLODActorData.TopLevelHLODActorNodes)
		{
			UpdateNodeState(HLODSceneNode);
		}
	}
}

void FWorldPartitionHLODEditorData::UpdateVisibility(const FVector& InCameraLocation, double InMinDrawDistance, double InMaxDrawDistance, bool bInForceVisibilityUpdate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FWorldPartitionHLODEditorData::UpdateVisibility);

	// Early out if no HLOD actor is loaded - none will be visible
	if (!HLODActorsLoader.IsValid())
	{
		return;
	}

	for (auto& MapEntry : PerContainerInstanceHLODActorDataMap)
	{
		FContainerInstanceHLODActorData& ContainerInstanceHLODActorData = MapEntry.Value;
		
		// For each top level HLOD actor (ex: HLOD2 in a 3 level of HLOD setup)
		// Determine visibility of each HLOD by a few factors:
		//  * If source (non-hlod) actors are loaded beneat it - HIDDEN
		//  * If near culled (using the min visible distance) - HIDDEN
		// Then if a given HLOD is HIDDEN, perform the same logic for its children.
		// If the HLOD is VISIBLE, then flag all it's children as HIDDEN
		for (const auto HLODActorNode : ContainerInstanceHLODActorData.TopLevelHLODActorNodes)
		{
			// Recurse from the top level HLOD down to HLOD0
			HLODActorNode->UpdateVisibility(InCameraLocation, InMinDrawDistance, InMaxDrawDistance, /*bInForceHidden*/false, bInForceVisibilityUpdate, LastStateUpdate);
		}
	}
}

void FHLODSceneNode::UpdateVisibility(const FVector& InCameraLocation, double InMinDrawDistance, double InMaxDrawDistance, bool bInForceHidden, bool bInForceVisibilityUpdate, int32 InLastStateUpdate)
{
	bool bNodeShouldBeVisible = !bInForceHidden;

	if (HLODActor)
	{
		// Do not show node if there are loaded actors under it
		if (bNodeShouldBeVisible)
		{
			const bool bHasLoadedActors = HasIntersectingLoadedRegion == InLastStateUpdate;
			bNodeShouldBeVisible = !bHasLoadedActors;
		}

		// Perform distance culling
		if (bNodeShouldBeVisible)
		{
			// Do not perform near culling if this HLOD has no child HLOD registered
			const bool bHasLoadedAndRegisteredChildrenHLODs = Algo::AnyOf(ChildrenHLODs, [](FHLODSceneNode* ChildHLODNode)
			{
				return ChildHLODNode->HLODActor != nullptr;
			});

			const bool bNeedsDistanceCheck = bHasLoadedAndRegisteredChildrenHLODs || (InMinDrawDistance != 0 || InMaxDrawDistance != 0);
			if (bNeedsDistanceCheck)
			{
				const double DistanceSquared = Bounds.ComputeSquaredDistanceFromBoxToPoint(InCameraLocation);

				if (bHasLoadedAndRegisteredChildrenHLODs)
				{
					const bool bNearCulled = DistanceSquared < FMath::Square(HLODActor->GetMinVisibleDistance());
					bNodeShouldBeVisible = !bNearCulled;
				}

				// Apply HLOD min/max draw distance user setting
				if (bNodeShouldBeVisible && (InMinDrawDistance != 0 || InMaxDrawDistance != 0))
				{
					const bool bNearCulled = InMinDrawDistance == 0 ? false : DistanceSquared < FMath::Square(InMinDrawDistance);
					const bool bFarCulled = InMaxDrawDistance == 0 ? false : DistanceSquared > FMath::Square(InMaxDrawDistance);
					bNodeShouldBeVisible = !bNearCulled && !bFarCulled;
				}
			}
		}

		if (bInForceVisibilityUpdate || bCachedIsVisible != bNodeShouldBeVisible)
		{
			bCachedIsVisible = bNodeShouldBeVisible;
			HLODActor->SetVisibility(bNodeShouldBeVisible);
		}
	}

	const bool bForceHideChildren = bNodeShouldBeVisible || bInForceHidden;
	for (FHLODSceneNode* Child : ChildrenHLODs)
	{
		Child->UpdateVisibility(InCameraLocation, InMinDrawDistance, InMaxDrawDistance, bForceHideChildren, bInForceVisibilityUpdate, InLastStateUpdate);
	}
}

void FWorldPartitionHLODEditorData::SetHLODLoadingState(bool bInShouldBeLoaded)
{
	if (bInShouldBeLoaded && !HLODActorsLoader.IsValid())
	{
		HLODActorsLoader.Reset(new FLoaderAdapterHLOD(WorldPartition->GetTypedOuter<UWorld>()));
	}
	else if (!bInShouldBeLoaded && HLODActorsLoader.IsValid())
	{
		HLODActorsLoader.Reset();
	}
}

void FWorldPartitionHLODEditorData::RegisterHLODActor(AWorldPartitionHLOD* Actor)
{
	const FGuid& ActorGuid = Actor->GetActorGuid();
	for (auto& [ContainerInstance, HLODActorData] : PerContainerInstanceHLODActorDataMap)
	{
		if (TUniquePtr<FHLODSceneNode>* NodePtr = HLODActorData.HLODActorNodes.Find(ActorGuid))
		{
			(*NodePtr)->HLODActor = Actor;
			return;
		}
	}
}

void FWorldPartitionHLODEditorData::UnregisterHLODActor(AWorldPartitionHLOD* Actor)
{
	const FGuid& ActorGuid = Actor->GetActorGuid();
	for (auto& [ContainerInstance, HLODActorData] : PerContainerInstanceHLODActorDataMap)
	{
		if (TUniquePtr<FHLODSceneNode>* NodePtr = HLODActorData.HLODActorNodes.Find(ActorGuid))
		{
			(*NodePtr)->HLODActor = nullptr;
			return;
		}
	}
}
