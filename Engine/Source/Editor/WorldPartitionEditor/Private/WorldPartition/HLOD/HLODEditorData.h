// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/HLOD/HLODLoaderAdapter.h"
#include "ExternalDirtyActorsTracker.h"

class AWorldPartitionHLOD;
class UWorldPartition;
class UActorDescContainerInstance;

// Represent an HLOD actor in the editor, loaded or not
struct FHLODSceneNode
{
	void UpdateVisibility(const FVector& InCameraLocation, double InMinDrawDistance, double InMaxDrawDistance, bool bInForceHidden, bool bInForceVisibilityUpdate, int32 InLastStateUpdate);

	FHLODSceneNode* ParentHLOD = nullptr;
	TArray<FHLODSceneNode*> ChildrenHLODs;


	FBoxSphereBounds Bounds;
	bool bCachedIsVisible = true;
	AWorldPartitionHLOD* HLODActor = nullptr;
	
	int32 HasIntersectingLoadedRegion = INDEX_NONE;
};

// Editor state of HLODs for a given World Partition
struct FWorldPartitionHLODEditorData
{
public:
	FWorldPartitionHLODEditorData(UWorldPartition* InWorldPartition);
	~FWorldPartitionHLODEditorData();
	
	bool IsLoadedActorsStateInitialized() const;
	void ClearLoadedActorsState();
	void UpdateLoadedActorsState();
	void UpdateVisibility(const FVector& InCameraLocation, double InMinDrawDistance, double InMaxDrawDistance, bool bForceVisibilityUpdate);

	void SetHLODLoadingState(bool bInShouldBeLoaded);

	void RegisterHLODActor(AWorldPartitionHLOD* Actor);
	void UnregisterHLODActor(AWorldPartitionHLOD* Actor);

private:
	void OnActorDescContainerInstanceRegistered(UActorDescContainerInstance* InContainerInstance);
	void OnActorDescContainerInstanceUnregistered(UActorDescContainerInstance* InContainerInstance);

private:
	struct FContainerInstanceHLODActorData
	{
		TMap<FGuid, TUniquePtr<FHLODSceneNode>> HLODActorNodes;
		TArray<FHLODSceneNode*> TopLevelHLODActorNodes;
	};

	UWorldPartition* WorldPartition;
	TMap<UActorDescContainerInstance*, FContainerInstanceHLODActorData> PerContainerInstanceHLODActorDataMap;
	TUniquePtr<FLoaderAdapterHLOD> HLODActorsLoader;
	int32 LastStateUpdate;

	struct FExternalDirtyActorTrackerGuid
	{
		using Type = FGuid;
		using OwnerType = FWorldPartitionHLODEditorData;
		static FGuid Store(FWorldPartitionHLODEditorData* InOwner, AActor* InActor) { return InActor->GetActorGuid(); }
	};

	using FExternalDirtyActorsTracker = TExternalDirtyActorsTracker<FExternalDirtyActorTrackerGuid>;
	TUniquePtr<FExternalDirtyActorsTracker> ExternalDirtyActorsTracker;
};
