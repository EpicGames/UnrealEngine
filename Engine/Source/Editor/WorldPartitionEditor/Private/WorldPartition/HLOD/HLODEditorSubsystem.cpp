// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODEditorSubsystem.h"

#include "Editor.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "GameFramework/ActorPrimitiveColorHandler.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelEditor.h"
#include "LevelEditorMenuContext.h"
#include "Misc/FileHelper.h"
#include "Misc/StringOutputDevice.h"
#include "SLevelViewport.h"
#include "StaticMeshResources.h"
#include "Subsystems/UnrealEditorSubsystem.h"
#include "ToolMenus.h"
#include "Widgets/Input/SSpinBox.h"
#include "WorldPartitionEditorModule.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODActorDesc.h"
#include "WorldPartition/HLOD/HLODEditorData.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartition/HLOD/HLODStats.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartitionEditorSettings.h"
#include "WorldPartition/WorldPartitionHLODsBuilder.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"

#include "PropertyPermissionList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODEditorSubsystem)

static TAutoConsoleVariable<bool> CVarHLODInEditorEnabled(
	TEXT("wp.Editor.HLOD.AllowShowingHLODsInEditor"),
	true,
	TEXT("Allow showing World Partition HLODs in the editor."));

#define LOCTEXT_NAMESPACE "HLODEditorSubsystem"

DEFINE_LOG_CATEGORY_STATIC(LogHLODEditorSubsystem, Log, All);

static FName NAME_HLODRelevantColorHandler(TEXT("HLODRelevantColorHandler"));
TMap<EHLODSettingsVisibility, UWorldPartitionHLODEditorSubsystem::FStructsPropertiesMap> UWorldPartitionHLODEditorSubsystem::StructsPropertiesVisibility;

UWorldPartitionHLODEditorSubsystem::UWorldPartitionHLODEditorSubsystem()
{
#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	if (HasAnyFlags(RF_ClassDefaultObject) && ExactCast<UWorldPartitionHLODEditorSubsystem>(this))
	{
		FActorPrimitiveColorHandler::FPrimitiveColorHandler HLODRelevantColorHandler;
		HLODRelevantColorHandler.HandlerName = NAME_HLODRelevantColorHandler;
		HLODRelevantColorHandler.HandlerText = LOCTEXT("HLODRelevantColor", "HLOD Relevant Color");
		HLODRelevantColorHandler.HandlerToolTipText = LOCTEXT("HLODRelevantColor_ToolTip", "Colorize actor if relevant to the HLOD system. Green means relevant, otherwise the color is Red.");
		HLODRelevantColorHandler.GetColorFunc = [](const UPrimitiveComponent* InPrimitiveComponent) -> FLinearColor
		{
			if (AActor* Actor = InPrimitiveComponent->GetOwner())
			{
				if (InPrimitiveComponent->IsHLODRelevant() && Actor->IsHLODRelevant())
				{
					return FLinearColor::Green;
				}
			}
			return FLinearColor::Red;
		};
		
		HLODRelevantColorHandler.ActivateFunc = [this]()
		{
			FCoreUObjectDelegates::OnObjectPropertyChanged.AddUObject(this, &UWorldPartitionHLODEditorSubsystem::OnColorHandlerPropertyChangedEvent);
		};

		HLODRelevantColorHandler.DeactivateFunc = [this]()
		{
			FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
		};

		FActorPrimitiveColorHandler::Get().RegisterPrimitiveColorHandler(HLODRelevantColorHandler);
	}
#endif

	if (IsTemplate())
	{
		HLOD_ADD_CLASS_SETTING_FILTER_NAME(BasicSettings, UHLODLayer, UHLODLayer::GetHLODBuilderSettingsPropertyName());
	}
}

UWorldPartitionHLODEditorSubsystem::~UWorldPartitionHLODEditorSubsystem()
{
#if ENABLE_ACTOR_PRIMITIVE_COLOR_HANDLER
	if (HasAnyFlags(RF_ClassDefaultObject) && ExactCast<UWorldPartitionHLODEditorSubsystem>(this))
	{
		FActorPrimitiveColorHandler::Get().UnregisterPrimitiveColorHandler(NAME_HLODRelevantColorHandler);
	}
#endif
}

void UWorldPartitionHLODEditorSubsystem::OnColorHandlerPropertyChangedEvent(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent)
{
	// When dealing with an LI, make sure to refresh the primitive color of all sub actors
	if (ILevelInstanceInterface* LevelInstanceInterface = Cast<ILevelInstanceInterface>(InObject))
	{
		if (UWorld* World = InObject->GetWorld())
		{
			ULevelInstanceSubsystem* const LevelInstanceSubsystem = World->GetSubsystem<ULevelInstanceSubsystem>();
			auto RefreshForLI = [LevelInstanceSubsystem](const ILevelInstanceInterface* LI)
			{
				ULevel* Level = LevelInstanceSubsystem->GetLevelInstanceLevel(LI);
				if (Level)
				{
					FActorPrimitiveColorHandler::Get().RefreshPrimitiveColorHandler(NAME_HLODRelevantColorHandler, Level->Actors);
				}
			};

			// Refresh LI actors
			RefreshForLI(LevelInstanceInterface);

			// Refresh child LIs actors
			LevelInstanceSubsystem->ForEachLevelInstanceChild(LevelInstanceInterface, /*bRecursive=*/true, [&RefreshForLI](ILevelInstanceInterface* ChildLevelInstance)
			{
				RefreshForLI(ChildLevelInstance);
				return true;
			});
		}
	}
}

bool UWorldPartitionHLODEditorSubsystem::IsHLODInEditorEnabled()
{
	if (IsRunningCommandlet())
	{
		return false;
	}

	if (!GEditor)
	{
		return false;
	}
	
	UWorld* CurrentWorld = GEditor->GetEditorWorldContext().World();
	const IWorldPartitionEditorModule* WorldPartitionEditorModule = FModuleManager::GetModulePtr<IWorldPartitionEditorModule>("WorldPartitionEditor");
	const bool bShowHLODsInEditorForWorld = WorldPartitionEditorModule && WorldPartitionEditorModule->IsHLODInEditorAllowed(CurrentWorld);
	const bool bShowHLODsInEditorUserSetting = WorldPartitionEditorModule && WorldPartitionEditorModule->GetShowHLODsInEditor();
	const bool bWorldPartitionLoadingInEditorEnabled = WorldPartitionEditorModule && WorldPartitionEditorModule->GetEnableLoadingInEditor();
	return CVarHLODInEditorEnabled.GetValueOnGameThread() && bShowHLODsInEditorUserSetting && bShowHLODsInEditorForWorld && bWorldPartitionLoadingInEditorEnabled;
}

void UWorldPartitionHLODEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	bForceHLODStateUpdate = true;
	bForceHLODVisibilityUpdate = true;
	CachedCameraLocation = FVector::Zero();
	CachedHLODMinDrawDistance = 0;
	CachedHLODMaxDrawDistance = 0;
	bCachedShowHLODsOverLoadedRegions = false;
	bHLODSettingsFilteringActive = false;
	
	FWorldDelegates::OnPostWorldInitialization.AddUObject(this, &UWorldPartitionHLODEditorSubsystem::OnPostWorldInitialization);

	GEngine->OnLevelActorListChanged().AddUObject(this, &UWorldPartitionHLODEditorSubsystem::ForceHLODStateUpdate);

	UWorldPartitionEditorSettings::OnSettingsChanged().AddUObject(this, &UWorldPartitionHLODEditorSubsystem::OnWorldPartitionEditorSettingsChanged);

	UWorldPartitionHLODsBuilder::GetOnWorldPartitionHLODBuildWithFiltersCompleted().AddUObject(this, &UWorldPartitionHLODEditorSubsystem::ForceHLODVisibilityUpdate);

	ApplyHLODSettingsFiltering();

	HLODActorEditorRegisteredHandle = AWorldPartitionHLOD::GetHLODActorEditorRegisteredDelegate().AddUObject(this, &UWorldPartitionHLODEditorSubsystem::OnHLODActorRegistered);
	HLODActorEditorUnregisteredHandle = AWorldPartitionHLOD::GetHLODActorEditorUnregisteredDelegate().AddUObject(this, &UWorldPartitionHLODEditorSubsystem::OnHLODActorUnregistered);

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	if (TSharedPtr<ILevelEditor> FirstLevelEditor = LevelEditorModule.GetFirstLevelEditor())
	{
		OnLevelEditorCreated(FirstLevelEditor);
	}
	else
	{
		LevelEditorModule.OnLevelEditorCreated().AddUObject(this, &UWorldPartitionHLODEditorSubsystem::OnLevelEditorCreated);
	}
}

void UWorldPartitionHLODEditorSubsystem::Deinitialize()
{
	Super::Deinitialize();

	AWorldPartitionHLOD::GetHLODActorEditorRegisteredDelegate().Remove(HLODActorEditorRegisteredHandle);
	AWorldPartitionHLOD::GetHLODActorEditorUnregisteredDelegate().Remove(HLODActorEditorUnregisteredHandle);

	UWorldPartitionHLODsBuilder::GetOnWorldPartitionHLODBuildWithFiltersCompleted().RemoveAll(this);

	UWorldPartitionEditorSettings::OnSettingsChanged().RemoveAll(this);

	GEngine->OnLevelActorListChanged().RemoveAll(this);

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorModule.OnLevelEditorCreated().RemoveAll(this);

	FWorldDelegates::OnPostWorldInitialization.RemoveAll(this);
}

void UWorldPartitionHLODEditorSubsystem::OnLevelEditorCreated(TSharedPtr<ILevelEditor> InLevelEditor)
{
	if (GEditor)
	{
		ExtendShowFlagsMenu();
	}
}

void UWorldPartitionHLODEditorSubsystem::ExtendShowFlagsMenu()
{
	check(GEditor);

	UToolMenu* const Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.ViewportToolbar.Show");
	if (!Menu)
	{
		return;
	}

	FToolMenuSection& AllShowFlagsSection = Menu->FindOrAddSection("AllShowFlags");

	// This is a dynamic entry so we can skip adding the submenu if the context
	// indicates that the viewport's world isn't partitioned.
	AllShowFlagsSection.AddEntry(FToolMenuEntry::InitDynamicEntry(
		"ShowHLODsDynamic",
		FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& InDynamicSection) -> void
			{
				TSharedPtr<SLevelViewport> Viewport = ULevelViewportContext::GetLevelViewport(InDynamicSection);
				if (!Viewport)
				{
					return;
				}

				UWorld* World = Viewport->GetWorld();
				if (!World)
				{
					return;
				}

				// Only add this submenu for partitioned worlds.
				if (!World->IsPartitionedWorld())
				{
					return;
				}

				InDynamicSection.AddSubMenu(
					"ShowHLODsMenu",
					LOCTEXT("ShowHLODsMenu", "HLODs"),
					LOCTEXT("ShowHLODsMenu_ToolTip", "Settings for HLODs in editor"),
					FNewToolMenuDelegate::CreateLambda(
						[](UToolMenu* Submenu)
						{
							TSharedPtr<SLevelViewport> Viewport = ULevelViewportContext::GetLevelViewport(Submenu);
							if (!Viewport)
							{
								return;
							}

							UWorld* World = Viewport->GetWorld();
							UWorldPartition* WorldPartition = World ? World->GetWorldPartition() : nullptr;
							if (!WorldPartition)
							{
								return;
							}

							IWorldPartitionEditorModule* WorldPartitionEditorModule =
								FModuleManager::GetModulePtr<IWorldPartitionEditorModule>("WorldPartitionEditor");
							if (!WorldPartitionEditorModule)
							{
								return;
							}

							FText HLODInEditorDisallowedReason;
							const bool bHLODInEditorAllowed =
								WorldPartitionEditorModule->IsHLODInEditorAllowed(World, &HLODInEditorDisallowedReason);

							// Show HLODs
							{
								FToolUIAction UIAction;
								UIAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda(
									[WorldPartitionEditorModule](const FToolMenuContext& InContext)
									{
										WorldPartitionEditorModule->SetShowHLODsInEditor(
											!WorldPartitionEditorModule->GetShowHLODsInEditor()
										);
									}
								);
								UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda(
									[bHLODInEditorAllowed](const FToolMenuContext& InContext)
									{
										return bHLODInEditorAllowed;
									}
								);
								UIAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda(
									[WorldPartitionEditorModule](const FToolMenuContext& InContext)
									{
										return WorldPartitionEditorModule->GetShowHLODsInEditor()
												 ? ECheckBoxState::Checked
												 : ECheckBoxState::Unchecked;
									}
								);
								FToolMenuEntry MenuEntry = FToolMenuEntry::InitMenuEntry(
									"ShowHLODs",
									LOCTEXT("ShowHLODs", "Show HLODs"),
									bHLODInEditorAllowed ? LOCTEXT("ShowHLODsToolTip", "Show/Hide HLODs")
														 : HLODInEditorDisallowedReason,
									FSlateIcon(),
									UIAction,
									EUserInterfaceActionType::ToggleButton
								);
								Submenu->AddMenuEntry(NAME_None, MenuEntry);
							}

							// Show HLODs Over Loaded Regions
							{
								FToolUIAction UIAction;
								UIAction.ExecuteAction = FToolMenuExecuteAction::CreateLambda(
									[WorldPartitionEditorModule](const FToolMenuContext& InContext)
									{
										WorldPartitionEditorModule->SetShowHLODsOverLoadedRegions(
											!WorldPartitionEditorModule->GetShowHLODsOverLoadedRegions()
										);
									}
								);
								UIAction.CanExecuteAction = FToolMenuCanExecuteAction::CreateLambda(
									[bHLODInEditorAllowed](const FToolMenuContext& InContext)
									{
										return bHLODInEditorAllowed;
									}
								);
								UIAction.GetActionCheckState = FToolMenuGetActionCheckState::CreateLambda(
									[WorldPartitionEditorModule](const FToolMenuContext& InContext)
									{
										return WorldPartitionEditorModule->GetShowHLODsOverLoadedRegions()
												 ? ECheckBoxState::Checked
												 : ECheckBoxState::Unchecked;
									}
								);
								FToolMenuEntry ShowHLODsEntry = FToolMenuEntry::InitMenuEntry(
									"ShowHLODsOverLoadedRegions",
									LOCTEXT("ShowHLODsOverLoadedRegions", "Show HLODs Over Loaded Regions"),
									bHLODInEditorAllowed
										? LOCTEXT("ShowHLODsOverLoadedRegions_ToolTip", "Show/Hide HLODs over loaded actors or regions")
										: HLODInEditorDisallowedReason,
									FSlateIcon(),
									UIAction,
									EUserInterfaceActionType::ToggleButton
								);
								Submenu->AddMenuEntry(NAME_None, ShowHLODsEntry);
							}

							// Min/Max Draw Distance
							{
								const double MinDrawDistanceMinValue = 0;
								const double MinDrawDistanceMaxValue = 102400;

								const double MaxDrawDistanceMinValue = 0;
								const double MaxDrawDistanceMaxValue = 1638400;

								// double SLevelViewportToolBar::OnGetHLODInEditorMinDrawDistanceValue() const
								auto OnGetHLODInEditorMinDrawDistanceValue = []() -> double
								{
									IWorldPartitionEditorModule* WorldPartitionEditorModule =
										FModuleManager::GetModulePtr<IWorldPartitionEditorModule>("WorldPartitionEditor");
									return WorldPartitionEditorModule
											 ? WorldPartitionEditorModule->GetHLODInEditorMinDrawDistance()
											 : 0;
								};

								// void SLevelViewportToolBar::OnHLODInEditorMinDrawDistanceValueChanged(double NewValue) const
								auto OnHLODInEditorMinDrawDistanceValueChanged = [](double NewValue) -> void
								{
									IWorldPartitionEditorModule* WorldPartitionEditorModule =
										FModuleManager::GetModulePtr<IWorldPartitionEditorModule>("WorldPartitionEditor");
									if (WorldPartitionEditorModule)
									{
										WorldPartitionEditorModule->SetHLODInEditorMinDrawDistance(NewValue);
										GEditor->RedrawLevelEditingViewports(true);
									}
								};

								TSharedRef<SSpinBox<double>> MinDrawDistanceSpinBox =
									SNew(SSpinBox<double>)
										.MinValue(MinDrawDistanceMinValue)
										.MaxValue(MinDrawDistanceMaxValue)
										.IsEnabled(bHLODInEditorAllowed)
										.Value_Lambda(OnGetHLODInEditorMinDrawDistanceValue)
										.OnValueChanged_Lambda(OnHLODInEditorMinDrawDistanceValueChanged)
										.ToolTipText(
											bHLODInEditorAllowed
												? LOCTEXT(
													  "HLODsInEditor_MinDrawDistance_Tooltip",
													  "Sets the minimum distance at which HLOD will be rendered"
												  )
												: HLODInEditorDisallowedReason
										)
										.OnBeginSliderMovement_Lambda(
											[]()
											{
												// Disable Slate throttling during slider drag to ensure immediate updates while moving the slider.
												FSlateThrottleManager::Get().DisableThrottle(true);
											}
										)
										.OnEndSliderMovement_Lambda(
											[](double)
											{
												FSlateThrottleManager::Get().DisableThrottle(false);
											}
										);

								// double SLevelViewportToolBar::OnGetHLODInEditorMaxDrawDistanceValue() const
								auto OnGetHLODInEditorMaxDrawDistanceValue = []() -> double
								{
									IWorldPartitionEditorModule* WorldPartitionEditorModule =
										FModuleManager::GetModulePtr<IWorldPartitionEditorModule>("WorldPartitionEditor");
									return WorldPartitionEditorModule
											 ? WorldPartitionEditorModule->GetHLODInEditorMaxDrawDistance()
											 : 0;
								};

								// void SLevelViewportToolBar::OnHLODInEditorMaxDrawDistanceValueChanged(double NewValue) const
								auto OnHLODInEditorMaxDrawDistanceValueChanged = [](double NewValue) -> void
								{
									IWorldPartitionEditorModule* WorldPartitionEditorModule =
										FModuleManager::GetModulePtr<IWorldPartitionEditorModule>("WorldPartitionEditor");
									if (WorldPartitionEditorModule)
									{
										WorldPartitionEditorModule->SetHLODInEditorMaxDrawDistance(NewValue);
										GEditor->RedrawLevelEditingViewports(true);
									}
								};

								TSharedRef<SSpinBox<double>> MaxDrawDistanceSpinBox =
									SNew(SSpinBox<double>)
										.MinValue(MaxDrawDistanceMinValue)
										.MaxValue(MaxDrawDistanceMaxValue)
										.IsEnabled(bHLODInEditorAllowed)
										.Value_Lambda(OnGetHLODInEditorMaxDrawDistanceValue)
										.OnValueChanged_Lambda(OnHLODInEditorMaxDrawDistanceValueChanged)
										.ToolTipText(
											bHLODInEditorAllowed
												? LOCTEXT(
													  "HLODsInEditor_MaxDrawDistance_Tooltip", "Sets the maximum distance at which HLODs will be rendered (0.0 means infinite)"
												  )
												: HLODInEditorDisallowedReason
										)
										.OnBeginSliderMovement_Lambda(
											[]()
											{
												// Disable Slate throttling during slider drag to ensure immediate updates while moving the slider.
												FSlateThrottleManager::Get().DisableThrottle(true);
											}
										)
										.OnEndSliderMovement_Lambda(
											[](double)
											{
												FSlateThrottleManager::Get().DisableThrottle(false);
											}
										);

								auto CreateDrawDistanceWidget = [](TSharedRef<SSpinBox<double>> InSpinBoxWidget)
								{
									// clang-format off
									return SNew(SBox)
										.HAlign(HAlign_Right)
										[
											SNew(SBox)
										  .Padding(FMargin(0.0f, 0.0f, 0.0f, 0.0f))
										  .WidthOverride(100.0f)
											[
												SNew(SBorder)
												.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
												.Padding(FMargin(1.0f))
												[
													InSpinBoxWidget
												]
											]
										];
									// clang-format on
								};

								FToolMenuEntry MinDrawDistanceMenuEntry = FToolMenuEntry::InitWidget(
									"Min Draw Distance",
									CreateDrawDistanceWidget(MinDrawDistanceSpinBox),
									LOCTEXT("MinDrawDistance", "Min Draw Distance")
								);
								Submenu->AddMenuEntry(NAME_None, MinDrawDistanceMenuEntry);

								FToolMenuEntry MaxDrawDistanceMenuEntry = FToolMenuEntry::InitWidget(
									"Max Draw Distance",
									CreateDrawDistanceWidget(MaxDrawDistanceSpinBox),
									LOCTEXT("MaxDrawDistance", "Max Draw Distance")
								);
								Submenu->AddMenuEntry(NAME_None, MaxDrawDistanceMenuEntry);
							}
						}
					),
					false,
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), "ShowFlagsMenu.SubMenu.HLODs")
				);
			}
		)
	));
}

void UWorldPartitionHLODEditorSubsystem::OnPostWorldInitialization(UWorld* InWorld, const UWorld::InitializationValues InIVS)
{
	if (InWorld && InWorld->WorldType == EWorldType::Editor)
	{
		check(!WorldPartitionsHLODEditorData.Contains(InWorld->GetWorldPartition()));
		InWorld->OnWorldPartitionInitialized().AddUObject(this, &UWorldPartitionHLODEditorSubsystem::OnWorldPartitionInitialized);
		InWorld->OnWorldPartitionUninitialized().AddUObject(this, &UWorldPartitionHLODEditorSubsystem::OnWorldPartitionUninitialized);
	}
}

void UWorldPartitionHLODEditorSubsystem::OnWorldPartitionEditorSettingsChanged(const FName& InPropertyName, const UWorldPartitionEditorSettings& InWorldPartitionEditorSettings)
{
	if (InPropertyName == UWorldPartitionEditorSettings::GetEnableAdvancedHLODSettingsPropertyName())
	{
		ApplyHLODSettingsFiltering();
	}
}

void UWorldPartitionHLODEditorSubsystem::ApplyHLODSettingsFiltering()
{
	static const FName PropertyPermissionListOwnerName = "AdvancedHLODSettingsFiltering";

	if (bHLODSettingsFilteringActive != !GetDefault<UWorldPartitionEditorSettings>()->GetEnableAdvancedHLODSettings())
	{
		bHLODSettingsFilteringActive = !GetDefault<UWorldPartitionEditorSettings>()->GetEnableAdvancedHLODSettings();

		if (!bHLODSettingsFilteringActive)
		{
			FPropertyEditorPermissionList::Get().UnregisterOwner(PropertyPermissionListOwnerName);
		}
		else
		{
			for (const TPair<TSoftObjectPtr<UStruct>, TSet<FName>>& StructProperties : StructsPropertiesVisibility.FindOrAdd(EHLODSettingsVisibility::BasicSettings))
			{
				FPropertyEditorPermissionList::Get().AddToAllowList(StructProperties.Key, StructProperties.Value.Array(), PropertyPermissionListOwnerName);
			}
		}
	}
}

void UWorldPartitionHLODEditorSubsystem::OnWorldPartitionInitialized(UWorldPartition* InWorldPartition)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionHLODEditorSubsystem::OnWorldPartitionInitialized);
	
	if (InWorldPartition->IsMainWorldPartition() || InWorldPartition->IsStandaloneHLODWorld())
	{
		InWorldPartition->LoaderAdapterStateChanged.AddUObject(this, &UWorldPartitionHLODEditorSubsystem::OnLoaderAdapterStateChanged);
		TPimplPtr<FWorldPartitionHLODEditorData>& HLODEditorData = WorldPartitionsHLODEditorData.Emplace(InWorldPartition, MakePimpl<FWorldPartitionHLODEditorData>(InWorldPartition));
		HLODEditorData->ClearLoadedActorsState();
		ForceHLODStateUpdate();
	}
}

void UWorldPartitionHLODEditorSubsystem::OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionHLODEditorSubsystem::OnWorldPartitionUninitialized);

	if (InWorldPartition->IsMainWorldPartition() || InWorldPartition->IsStandaloneHLODWorld())
	{
		InWorldPartition->LoaderAdapterStateChanged.RemoveAll(this);
		WorldPartitionsHLODEditorData.Remove(InWorldPartition);
	}
}

void UWorldPartitionHLODEditorSubsystem::OnLoaderAdapterStateChanged(const IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionHLODEditorSubsystem::OnLoaderAdapterStateChanged);

	ForceHLODStateUpdate();
}

void UWorldPartitionHLODEditorSubsystem::OnHLODActorRegistered(AWorldPartitionHLOD* Actor)
{
	if (UWorld* World = Actor->GetWorld())
	{
		if (UWorldPartition* WorldPartition = World->GetWorldPartition())
		{
			if (TPimplPtr<FWorldPartitionHLODEditorData>* HLODEditorData = WorldPartitionsHLODEditorData.Find(WorldPartition))
			{
				(*HLODEditorData)->RegisterHLODActor(Actor);
			}
		}
	}
}

void UWorldPartitionHLODEditorSubsystem::OnHLODActorUnregistered(AWorldPartitionHLOD* Actor)
{
	if (UWorld* World = Actor->GetWorld())
	{
		if (UWorldPartition* WorldPartition = World->GetWorldPartition())
		{
			if (TPimplPtr<FWorldPartitionHLODEditorData>* HLODEditorData = WorldPartitionsHLODEditorData.Find(WorldPartition))
			{
				(*HLODEditorData)->UnregisterHLODActor(Actor);
			}
		}
	}
}

void UWorldPartitionHLODEditorSubsystem::ForceHLODVisibilityUpdate()
{
	if (IsHLODInEditorEnabled())
	{
		bForceHLODVisibilityUpdate = true;
	}
}

void UWorldPartitionHLODEditorSubsystem::ForceHLODStateUpdate()
{
	if (IsHLODInEditorEnabled())
	{
		bForceHLODStateUpdate = true;
	}
}

bool UWorldPartitionHLODEditorSubsystem::IsTickable() const
{
	return GEditor != nullptr;
}

void UWorldPartitionHLODEditorSubsystem::Tick(float DeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionHLODEditorSubsystem::Tick);

	UUnrealEditorSubsystem* UnrealEditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>();

	bool bCameraMoved = false;
	bool bClearLoadedActorState = false;

	// Check cached global settings
	if (IsHLODInEditorEnabled())
	{
		IWorldPartitionEditorModule* WorldPartitionEditorModule = FModuleManager::GetModulePtr<IWorldPartitionEditorModule>("WorldPartitionEditor");

		// "Show HLODs over loaded region" option changed ?
		if (WorldPartitionEditorModule->GetShowHLODsOverLoadedRegions() != bCachedShowHLODsOverLoadedRegions)
		{
			bCachedShowHLODsOverLoadedRegions = WorldPartitionEditorModule->GetShowHLODsOverLoadedRegions();
			bForceHLODVisibilityUpdate = true;
			bForceHLODStateUpdate = true;
			bClearLoadedActorState = bCachedShowHLODsOverLoadedRegions;
		}

		// Min/Max draw distance for HLODs was changed ?
		if (WorldPartitionEditorModule->GetHLODInEditorMinDrawDistance() != CachedHLODMinDrawDistance ||
			WorldPartitionEditorModule->GetHLODInEditorMaxDrawDistance() != CachedHLODMaxDrawDistance)
		{
			CachedHLODMinDrawDistance = WorldPartitionEditorModule->GetHLODInEditorMinDrawDistance();
			CachedHLODMaxDrawDistance = WorldPartitionEditorModule->GetHLODInEditorMaxDrawDistance();
			bForceHLODVisibilityUpdate = true;
		}

		if (UnrealEditorSubsystem)
		{
			FVector CameraLocation;
			FRotator CameraRotation;

			if (FWorldPartitionEditorModule::GetActiveLevelViewportCameraInfo(CameraLocation, CameraRotation))
			{
				// Camera was moved ?
				bCameraMoved = CameraLocation != CachedCameraLocation;
				if (bCameraMoved)
				{
					CachedCameraLocation = CameraLocation;
				}
			}
		}
	}

	for (TPair<TObjectKey<UWorldPartition>, TPimplPtr<FWorldPartitionHLODEditorData>>& Pair : WorldPartitionsHLODEditorData)
	{
		TPimplPtr<FWorldPartitionHLODEditorData>& HLODEditorData = Pair.Value;

		HLODEditorData->SetHLODLoadingState(IsHLODInEditorEnabled());
		
		if (IsHLODInEditorEnabled())
		{
			bool bNeedsInitialization = !HLODEditorData->IsLoadedActorsStateInitialized();

			if (bClearLoadedActorState || (bNeedsInitialization && bCachedShowHLODsOverLoadedRegions))
			{
				HLODEditorData->ClearLoadedActorsState();
			}

			// Actors or regions were loaded ?
			if ((bForceHLODStateUpdate || bNeedsInitialization) && !bCachedShowHLODsOverLoadedRegions)
			{
				HLODEditorData->UpdateLoadedActorsState();
				bForceHLODVisibilityUpdate = true;
			}

			if (bForceHLODVisibilityUpdate || bCameraMoved || bNeedsInitialization)
			{
				HLODEditorData->UpdateVisibility(CachedCameraLocation, CachedHLODMinDrawDistance, CachedHLODMaxDrawDistance, bForceHLODVisibilityUpdate);
			}
		}
	}
	bForceHLODStateUpdate = false;
	bForceHLODVisibilityUpdate = false;
}

TStatId UWorldPartitionHLODEditorSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(WorldPartitionHLODEditorSubsystem, STATGROUP_Tickables);
}

void UWorldPartitionHLODEditorSubsystem::AddHLODSettingsFilter(EHLODSettingsVisibility InSettingsVisibility, TSoftObjectPtr<UStruct> InStruct, FName InPropertyName)
{
	StructsPropertiesVisibility.FindOrAdd(InSettingsVisibility).FindOrAdd(InStruct).Add(InPropertyName);
}

bool UWorldPartitionHLODEditorSubsystem::WriteHLODStats(const IWorldPartitionEditorModule::FWriteHLODStatsParams& Params)
{
	bool bResult = false;

	switch(Params.StatsType)
	{
	case IWorldPartitionEditorModule::FWriteHLODStatsParams::EStatsType::Default:
		bResult = WriteHLODStats(Params.World, Params.Filename);
		break;

	case IWorldPartitionEditorModule::FWriteHLODStatsParams::EStatsType::InputDetails:
		bResult = WriteHLODInputStats(Params.World, Params.Filename);
		break;
	}

	if (bResult)
	{
		UE_LOGF(LogHLODEditorSubsystem, Display, "Wrote HLOD stats to %ls", *Params.Filename);
	}
	else
	{
		UE_LOGF(LogHLODEditorSubsystem, Error, "Failed to write HLOD stats to %ls", *Params.Filename);
	}

	return bResult;
}

bool UWorldPartitionHLODEditorSubsystem::WriteHLODStats(UWorld* InWorld, const FString& InFilename)
{
	UWorldPartition* WorldPartition = InWorld->GetWorldPartition();
	if (!WorldPartition)
	{
		return false;
	}

	typedef TFunction<FString(FWorldPartitionActorDescInstance* InActorDescInstance, const FHLODActorDesc&)> FGetStatFunc;

	auto GetHLODStat = [](FName InStatName)
	{
		return TPair<FName, FGetStatFunc>(InStatName, [InStatName](FWorldPartitionActorDescInstance* InActorDescInstance, const FHLODActorDesc& InActorDesc)
		{
			return FString::Printf(TEXT("%lld"), InActorDesc.GetStat(InStatName));
		});
	};

	const UDataLayerManager* DataLayerManager = WorldPartition->GetDataLayerManager();

	auto GetDataLayerShortName = [DataLayerManager](FName DataLayerInstanceName)
	{
		const UDataLayerInstance* DataLayerInstance = DataLayerManager ? DataLayerManager->GetDataLayerInstance(DataLayerInstanceName) : nullptr;
		return DataLayerInstance ? DataLayerInstance->GetDataLayerShortName() : DataLayerInstanceName.ToString();
	};

	TArray<TPair<FName, FGetStatFunc>> StatsToWrite =
	{
		{ "WorldPackage",		[InWorld](FWorldPartitionActorDescInstance* InActorDescInstance, const FHLODActorDesc& InActorDesc) { return InWorld->GetPackage()->GetName(); } },
		{ "Name",				[](FWorldPartitionActorDescInstance* InActorDescInstance, const FHLODActorDesc& InActorDesc) { return InActorDescInstance->GetActorLabelString(); } },
		{ "HLODLayer",			[](FWorldPartitionActorDescInstance* InActorDescInstance, const FHLODActorDesc& InActorDesc) { return InActorDesc.GetSourceHLODLayer().GetAssetName().ToString(); }},
		{ "SpatiallyLoaded",	[](FWorldPartitionActorDescInstance* InActorDescInstance, const FHLODActorDesc& InActorDesc) { return InActorDescInstance->GetIsSpatiallyLoaded() ? TEXT("true") : TEXT("false"); } },
		{ "DataLayers",			[&GetDataLayerShortName](FWorldPartitionActorDescInstance* InActorDescInstance, const FHLODActorDesc& InActorDesc) { return FString::JoinBy(InActorDescInstance->GetDataLayerInstanceNames().ToArray(), TEXT(" | "), GetDataLayerShortName); }},

		GetHLODStat(FWorldPartitionHLODStats::InputActorCount),
		GetHLODStat(FWorldPartitionHLODStats::InputTriangleCount),
		GetHLODStat(FWorldPartitionHLODStats::InputVertexCount),

		GetHLODStat(FWorldPartitionHLODStats::MeshInstanceCount),
		GetHLODStat(FWorldPartitionHLODStats::MeshNaniteTriangleCount),
		GetHLODStat(FWorldPartitionHLODStats::MeshNaniteVertexCount),
		GetHLODStat(FWorldPartitionHLODStats::MeshTriangleCount),
		GetHLODStat(FWorldPartitionHLODStats::MeshVertexCount),
		GetHLODStat(FWorldPartitionHLODStats::MeshUVChannelCount),

		GetHLODStat(FWorldPartitionHLODStats::MaterialBaseColorTextureSize),
		GetHLODStat(FWorldPartitionHLODStats::MaterialNormalTextureSize),
		GetHLODStat(FWorldPartitionHLODStats::MaterialEmissiveTextureSize),
		GetHLODStat(FWorldPartitionHLODStats::MaterialMetallicTextureSize),
		GetHLODStat(FWorldPartitionHLODStats::MaterialRoughnessTextureSize),
		GetHLODStat(FWorldPartitionHLODStats::MaterialSpecularTextureSize),

		GetHLODStat(FWorldPartitionHLODStats::MemoryMeshResourceSizeBytes),
		GetHLODStat(FWorldPartitionHLODStats::MemoryTexturesResourceSizeBytes),
		GetHLODStat(FWorldPartitionHLODStats::MemoryDiskSizeBytes),

		GetHLODStat(FWorldPartitionHLODStats::BuildTimeLoadMilliseconds),
		GetHLODStat(FWorldPartitionHLODStats::BuildTimeBuildMilliseconds),
		GetHLODStat(FWorldPartitionHLODStats::BuildTimeTotalMilliseconds)
	};

	FStringOutputDevice Output;

	// Write header if file doesn't exist
	if (!IFileManager::Get().FileExists(*InFilename))
	{
		const FString StatHeader = FString::JoinBy(StatsToWrite, TEXT(","), [](const TPair<FName, FGetStatFunc>& Pair) { return Pair.Key.ToString(); });
		Output.Logf(TEXT("%s" LINE_TERMINATOR_ANSI), *StatHeader);
	}

	// Write one line per HLOD actor desc
	for (FActorDescContainerInstanceCollection::TIterator<AWorldPartitionHLOD> HLODIterator(WorldPartition); HLODIterator; ++HLODIterator)
	{
		const FString StatLine = FString::JoinBy(StatsToWrite, TEXT(","), [&HLODIterator](const TPair<FName, FGetStatFunc>& Pair)
			{
				const FHLODActorDesc& HLODActorDesc = *(FHLODActorDesc*)HLODIterator->GetActorDesc();
				return Pair.Value(*HLODIterator, HLODActorDesc);
			});
		Output.Logf(TEXT("%s" LINE_TERMINATOR_ANSI), *StatLine);
	}

	// Write to file
	return FFileHelper::SaveStringToFile(Output, *InFilename, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
}

bool UWorldPartitionHLODEditorSubsystem::WriteHLODInputStats(UWorld* InWorld, const FString& InFilename)
{
	UWorldPartition* WorldPartition = InWorld->GetWorldPartition();
	if (!WorldPartition)
	{
		return false;
	}

	FWorldPartitionHelpers::FForEachActorWithLoadingParams ForEachActorWithLoadingParams;
	ForEachActorWithLoadingParams.ActorClasses = { AWorldPartitionHLOD::StaticClass() };

	TMap<TPair<int32, FName>, FHLODBuildInputReferencedAssets> BuildersReferencedAssets;

	// Aggregate referenced assets from all HLOD actors
	FWorldPartitionHelpers::ForEachActorWithLoading(WorldPartition, [WorldPartition, &BuildersReferencedAssets](const FWorldPartitionActorDescInstance* ActorDescInstance)
	{
		AWorldPartitionHLOD* HLODActor = Cast<AWorldPartitionHLOD>(ActorDescInstance->GetActor());
		if (!HLODActor)
		{
			UE_LOGF(LogHLODEditorSubsystem, Error, "HLOD actor failed to load: %ls (%ls)", *ActorDescInstance->GetActorNameString(), *ActorDescInstance->GetActorPackage().ToString());
			return false;
		}

		const FHLODBuildInputStats& InputStats = HLODActor->GetInputStats();

		for (const TPair<FName, FHLODBuildInputReferencedAssets>& Entry : InputStats.BuildersReferencedAssets)
		{
			FHLODBuildInputReferencedAssets& BuilderReferencedAssets = BuildersReferencedAssets.FindOrAdd(TPair<int32, FName>(HLODActor->GetLODLevel(), Entry.Key));
			for (const TPair<FTopLevelAssetPath, uint32>& ReferencedMesh : Entry.Value.StaticMeshes)
			{
				BuilderReferencedAssets.StaticMeshes.FindOrAdd(ReferencedMesh.Key) += ReferencedMesh.Value;
			}
		}

		return true;
	}, ForEachActorWithLoadingParams);

	FStringOutputDevice Output;

	Output.Logf(TEXT("HLODLevel,BuilderName,AssetName,RefCount,LastLODTriCount,LastLODVtxCount" LINE_TERMINATOR_ANSI));

	BuildersReferencedAssets.KeySort([](const TPair<int32, FName>& PairA, const TPair<int32, FName> PairB)
	{
		if (PairA.Key != PairB.Key)
		{
			return PairA.Key < PairB.Key;
		}

		return PairA.Value.LexicalLess(PairB.Value);
	});

	for (TPair<TPair<int32,FName>, FHLODBuildInputReferencedAssets>& Entry : BuildersReferencedAssets)
	{
		Entry.Value.StaticMeshes.KeySort(FTopLevelAssetPathFastLess());

		for (const TPair<FTopLevelAssetPath, uint32>& ReferencedMesh : Entry.Value.StaticMeshes)
		{
			const FTopLevelAssetPath& StaticMeshAssetPath = ReferencedMesh.Key;
			
			UObject* LoadedObject = StaticLoadAsset(UObject::StaticClass(), StaticMeshAssetPath, LOAD_NoWarn);
			if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(LoadedObject))
			{
				const FStaticMeshRenderData* RenderData = StaticMesh->GetRenderData();
				const bool bHasRenderData = RenderData && !RenderData->LODResources.IsEmpty();
				if (bHasRenderData)
				{
					const int32 LODIndex = StaticMesh->GetNumLODs() - 1;
					const int64 LastLODTriCount = RenderData->LODResources[LODIndex].GetNumTriangles();
					const int64 LastLODVtxCount = RenderData->LODResources[LODIndex].GetNumVertices();
					Output.Logf(TEXT("HLOD%d,%s,%s,%d,%d,%d" LINE_TERMINATOR_ANSI), Entry.Key.Key, *Entry.Key.Value.ToString(), *StaticMeshAssetPath.GetPackageName().ToString(), ReferencedMesh.Value, LastLODTriCount, LastLODVtxCount);
				}				
			}
		}
	}

	// Write to file
	return FFileHelper::SaveStringToFile(Output, *InFilename);
}

FAutoConsoleCommand HLODDumpStats(
	TEXT("wp.Editor.HLOD.DumpStats"),
	TEXT("Export various HLOD stats to a CSV formatted file."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		const FString HLODStatsOutputFilename = FPaths::ProjectLogDir() / TEXT("WorldPartition") / FString::Printf(TEXT("HLODStats-%08x-%s.csv"), FPlatformProcess::GetCurrentProcessId(), *FDateTime::Now().ToString());

		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (UWorld* World = Context.World())
			{
				IWorldPartitionEditorModule::FWriteHLODStatsParams Params;
				Params.World = World;
				Params.StatsType = IWorldPartitionEditorModule::FWriteHLODStatsParams::EStatsType::Default;
				Params.Filename = HLODStatsOutputFilename;
				UWorldPartitionHLODEditorSubsystem::WriteHLODStats(Params);
			}
		}
	})
);

FAutoConsoleCommand HLODDumpInputStats(
	TEXT("wp.Editor.HLOD.DumpInputStats"),
	TEXT("Export stats regarding the input to HLOD generation to a CSV formatted file."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		const FString HLODStatsOutputFilename = FPaths::ProjectLogDir() / TEXT("WorldPartition") / FString::Printf(TEXT("HLODInputStats-%08x-%s.csv"), FPlatformProcess::GetCurrentProcessId(), *FDateTime::Now().ToString());

		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (UWorld* World = Context.World())
			{
				IWorldPartitionEditorModule::FWriteHLODStatsParams Params;
				Params.World = World;
				Params.StatsType = IWorldPartitionEditorModule::FWriteHLODStatsParams::EStatsType::InputDetails;
				Params.Filename = HLODStatsOutputFilename;
				UWorldPartitionHLODEditorSubsystem::WriteHLODStats(Params);
			}
		}
	})
);

#undef LOCTEXT_NAMESPACE
