// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODLayerFactory.h"

#include "Misc/AssertionMacros.h"
#include "Templates/SubclassOf.h"
#include "WorldPartition/HLOD/HLODLayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(HLODLayerFactory)


UHLODLayerFactory::UHLODLayerFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SupportedClass = UHLODLayer::StaticClass();

	bCreateNew = true;
	bEditAfterNew = true;
	bEditorImport = false;
}

UObject* UHLODLayerFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	UHLODLayer* HLODLayer = nullptr;

	if (UHLODLayer* HLODLayerTemplate = UHLODLayer::GetEngineDefaultHLODLayersSetup())
	{
		HLODLayer = DuplicateObject<UHLODLayer>(HLODLayerTemplate, InParent, Name);
		HLODLayer->ClearFlags(RF_AllFlags);
		HLODLayer->SetFlags(Flags);

		// Make sure to never assign a parent HLOD layer to a newly created asset, even if the default template has one
		HLODLayer->SetParentLayer(nullptr);
	}
	
	if (!HLODLayer)
	{
		HLODLayer = NewObject<UHLODLayer>(InParent, Class, Name, Flags);
	}
	
	check(HLODLayer);
	return HLODLayer;
}
