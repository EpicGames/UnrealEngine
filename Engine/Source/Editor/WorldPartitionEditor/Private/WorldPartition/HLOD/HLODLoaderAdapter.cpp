// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/HLODLoaderAdapter.h"

#include "Engine/World.h"

#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODActorDesc.h"
#include "WorldPartition/HLOD/HLODLayer.h"

#include "WorldPartition/ActorDescContainerInstanceCollection.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"


FLoaderAdapterHLOD::FLoaderAdapterHLOD(UWorld* InWorld)
	: ILoaderAdapter(InWorld)
{
	Load();
}

void FLoaderAdapterHLOD::ForEachActor(TFunctionRef<void(const FWorldPartitionHandle&)> InOperation) const
{
	UWorldPartition* WorldPartition = GetWorld()->GetWorldPartition();

	for (FActorDescContainerInstanceCollection::TIterator<AWorldPartitionHLOD> HLODIterator(WorldPartition); HLODIterator; ++HLODIterator)
	{
		InOperation(FWorldPartitionHandle(WorldPartition, HLODIterator->GetGuid()));
	}
}

bool FLoaderAdapterHLOD::PassActorDescFilter(const FWorldPartitionHandle& ActorHandle) const
{
	// Do not call the base class implementation of PassActorDescFilter() here, as it will skip actors which are not editor relevant

	const FHLODActorDesc* HLODActorDesc = static_cast<const FHLODActorDesc*>(ActorHandle->GetActorDesc());
	if (HLODActorDesc)
	{
		FSoftObjectPath HLODLayerPath(HLODActorDesc->GetSourceHLODLayer());
		if (UHLODLayer* HLODLayer = Cast<UHLODLayer>(HLODLayerPath.TryLoad()))
		{
			switch (HLODLayer->GetEditorLoadingBehavior())
			{
			case EHLODLayerEditorLoadingBehavior::ForceLoaded:
				return true;
			case EHLODLayerEditorLoadingBehavior::ForceNotLoaded:
				return false;
			case EHLODLayerEditorLoadingBehavior::Default:
			default:
				if (!HLODActorDesc->GetIsSpatiallyLoaded() || HLODLayer->GetLayerType() != EHLODLayerType::Instancing)
				{
					return true;
				}
			}
		}
	}

	return false;
}
