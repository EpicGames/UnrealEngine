// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Layout/Clipping.h"
#include "Rendering/DrawElements.h"

/**
 * A simple compound widget that clips its single child to show only the portion
 * right of a dynamic wipe position. Used for the HLOD compare wipe overlay.
 */
class SHLODClipWrapper : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SHLODClipWrapper)
		: _WipePosition(0.5f)
	{}
		SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_ATTRIBUTE(float, WipePosition)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		WipePosition = InArgs._WipePosition;

		ChildSlot
		[
			InArgs._Content.Widget
		];

		SetVisibility(EVisibility::HitTestInvisible);
	}

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
	{
		const FVector2D AbsPos = AllottedGeometry.GetAbsolutePosition();
		const FVector2D AbsSize = AllottedGeometry.GetAbsoluteSize();
		const float WipeAbsX = AbsPos.X + AbsSize.X * WipePosition.Get();

		FSlateRect ClipRect(WipeAbsX, AbsPos.Y, AbsPos.X + AbsSize.X, AbsPos.Y + AbsSize.Y);
		FSlateClippingZone ClipZone(ClipRect);
		ClipZone.SetShouldIntersectParent(true);

		OutDrawElements.PushClip(ClipZone);
		int32 Result = SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
		OutDrawElements.PopClip();

		return Result;
	}

private:
	TAttribute<float> WipePosition;
};
