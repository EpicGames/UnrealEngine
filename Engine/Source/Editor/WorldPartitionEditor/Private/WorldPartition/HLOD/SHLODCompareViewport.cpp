// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/SHLODCompareViewport.h"
#include "WorldPartition/HLOD/HLODCompareViewportClient.h"
#include "WorldPartition/HLOD/SHLODCompareWindow.h"
#include "PreviewScene.h"
#include "ToolMenus.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "WorldPartition/HLOD/HLODCompareCommands.h"

#define LOCTEXT_NAMESPACE "SHLODCompareViewport"

static void PopulateHLODCompareViewModesMenu(UToolMenu* InMenu)
{
	const FHLODCompareCommands& Commands = FHLODCompareCommands::Get();
	const FName StyleSetName = FAppStyle::GetAppStyleSetName();

	auto ViewModeIcon = [&StyleSetName](const TCHAR* IconName) -> FSlateIcon
	{
		return FSlateIcon(StyleSetName, IconName);
	};

	// View modes section: Lit, Wireframe
	{
		FToolMenuSection& Section = InMenu->AddSection("ViewMode", LOCTEXT("ViewModeHeader", "View Mode"));
		Section.AddMenuEntry(Commands.ViewModeLit, TAttribute<FText>(), TAttribute<FText>(), ViewModeIcon(TEXT("EditorViewport.LitMode")));
		Section.AddMenuEntry(Commands.ViewModeWireframe, TAttribute<FText>(), TAttribute<FText>(), ViewModeIcon(TEXT("EditorViewport.WireframeMode")));
	}

	// Buffer visualization section
	{
		FSlateIcon BufferIcon = ViewModeIcon(TEXT("EditorViewport.VisualizeBufferMode"));
		FToolMenuSection& Section = InMenu->AddSection("BufferVisualization", LOCTEXT("BufferVisualizationHeader", "Buffer Visualization"));
		Section.AddMenuEntry(Commands.ViewModeBaseColor, TAttribute<FText>(), TAttribute<FText>(), BufferIcon);
		Section.AddMenuEntry(Commands.ViewModeMetallic, TAttribute<FText>(), TAttribute<FText>(), BufferIcon);
		Section.AddMenuEntry(Commands.ViewModeRoughness, TAttribute<FText>(), TAttribute<FText>(), BufferIcon);
		Section.AddMenuEntry(Commands.ViewModeSpecular, TAttribute<FText>(), TAttribute<FText>(), BufferIcon);
		Section.AddMenuEntry(Commands.ViewModeWorldNormal, TAttribute<FText>(), TAttribute<FText>(), BufferIcon);
	}
}

void SHLODCompareViewport::Construct(const FArguments& InArgs)
{
	PreviewScene = InArgs._PreviewScene;
	CompareWindow = InArgs._CompareWindow;

	SEditorViewport::Construct(SEditorViewport::FArguments());
}

FReply SHLODCompareViewport::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// Check the priority command list before the viewport's own commands.
	// This ensures our HLOD compare shortcuts (Alt+1..7, Space) take precedence
	// over the standard viewport view mode shortcuts.
	if (PriorityCommandList.IsValid() && PriorityCommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	// Swallow other Alt+number combos so the viewport's standard bindings don't fire
	if (InKeyEvent.IsAltDown())
	{
		const FKey Key = InKeyEvent.GetKey();
		if (Key == EKeys::Eight || Key == EKeys::Nine || Key == EKeys::Zero)
		{
			return FReply::Handled();
		}
	}

	return SEditorViewport::OnKeyDown(MyGeometry, InKeyEvent);
}

TSharedRef<FEditorViewportClient> SHLODCompareViewport::MakeEditorViewportClient()
{
	CompareViewportClient = MakeShareable(new FHLODCompareViewportClient(PreviewScene.Get(), SharedThis(this), CompareWindow));
	return CompareViewportClient.ToSharedRef();
}

TSharedRef<SWidget> SHLODCompareViewport::BuildExternalViewModeToolbar(const TSharedPtr<FUICommandList>& InCommandList)
{
	const FName ViewportToolbarName = "HLODCompare.ViewportToolbar";

	if (!UToolMenus::Get()->IsMenuRegistered(ViewportToolbarName))
	{
		UToolMenu* ViewportToolbarMenu = UToolMenus::Get()->RegisterMenu(
			ViewportToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar
		);
		ViewportToolbarMenu->StyleName = "ViewportToolbar";

		FToolMenuSection& ToolbarSection = ViewportToolbarMenu->AddSection("Right");
		ToolbarSection.Alignment = EToolMenuSectionAlign::Last;

		// Add a dynamic submenu entry with label/icon that reflects the current view mode
		ToolbarSection.AddEntry(FToolMenuEntry::InitDynamicEntry("DynamicViewModes",
			FNewToolMenuSectionDelegate::CreateLambda(
				[](FToolMenuSection& InSection)
				{
					UUnrealEdViewportToolbarContext* Context = InSection.FindContext<UUnrealEdViewportToolbarContext>();
					if (!Context)
					{
						return;
					}

					TAttribute<FText> LabelAttribute = TAttribute<FText>::CreateLambda(
						[WeakViewport = Context->Viewport]()
						{
							return UE::UnrealEd::GetViewModesSubmenuLabel(WeakViewport);
						});

					TAttribute<FSlateIcon> IconAttribute = TAttribute<FSlateIcon>::CreateLambda(
						[WeakViewport = Context->Viewport]()
						{
							return UE::UnrealEd::GetViewModesSubmenuIcon(WeakViewport);
						});

					InSection.AddSubMenu(
						"ViewModes",
						LabelAttribute,
						LOCTEXT("ViewModesSubmenuTooltip", "View mode settings for the current viewport."),
						FNewToolMenuDelegate::CreateStatic(&PopulateHLODCompareViewModesMenu),
						false,
						IconAttribute
					);
				})));
	}

	FToolMenuContext ViewportToolbarContext;
	{
		if (InCommandList.IsValid())
		{
			ViewportToolbarContext.AppendCommandList(InCommandList);
		}

		UUnrealEdViewportToolbarContext* ContextObject =
			UE::UnrealEd::CreateViewportToolbarDefaultContext(SharedThis(this));
		ContextObject->bShowCoordinateSystemControls = false;
		ViewportToolbarContext.AddObject(ContextObject);
	}

	return UToolMenus::Get()->GenerateWidget(ViewportToolbarName, ViewportToolbarContext);
}

#undef LOCTEXT_NAMESPACE
