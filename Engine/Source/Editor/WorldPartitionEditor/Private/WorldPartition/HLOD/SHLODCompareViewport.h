// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SEditorViewport.h"

class FPreviewScene;
class SHLODCompareWindow;
class FHLODCompareViewportClient;

class SHLODCompareViewport : public SEditorViewport
{
public:
	SLATE_BEGIN_ARGS(SHLODCompareViewport) {}
		SLATE_ARGUMENT(TSharedPtr<FPreviewScene>, PreviewScene)
		SLATE_ARGUMENT(TWeakPtr<SHLODCompareWindow>, CompareWindow)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	TSharedPtr<FHLODCompareViewportClient> GetCompareViewportClient() const { return CompareViewportClient; }

	/** Build a standalone view mode toolbar widget for this viewport (used by SHLODCompareWindow). */
	TSharedRef<SWidget> BuildExternalViewModeToolbar(const TSharedPtr<FUICommandList>& InCommandList);

	/** Set the priority command list that is checked before the viewport's own commands. */
	void SetPriorityCommandList(const TSharedPtr<FUICommandList>& InCommandList) { PriorityCommandList = InCommandList; }

protected:
	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override { return nullptr; }
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

private:
	TSharedPtr<FPreviewScene> PreviewScene;
	TWeakPtr<SHLODCompareWindow> CompareWindow;
	TSharedPtr<FHLODCompareViewportClient> CompareViewportClient;
	TSharedPtr<FUICommandList> PriorityCommandList;
};
