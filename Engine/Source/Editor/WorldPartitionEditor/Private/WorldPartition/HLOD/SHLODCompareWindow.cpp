// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/SHLODCompareWindow.h"
#include "WorldPartition/HLOD/SHLODCompareViewport.h"
#include "WorldPartition/HLOD/HLODCompareViewportClient.h"
#include "WorldPartition/HLOD/SHLODClipWrapper.h"
#include "WorldPartition/HLOD/SHLODWipeHandle.h"
#include "PreviewScene.h"
#include "Components/PrimitiveComponent.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/SkyLightComponent.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/ScopedSlowTask.h"
#include "LevelEditorViewport.h"
#include "Engine/World.h"
#include "GameFramework/WorldSettings.h"
#include "EngineUtils.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SCanvas.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODCompareCommands.h"
#include "WorldPartition/HLOD/HLODSourceActors.h"
#include "WorldPartition/HLOD/SHLODDistanceRuler.h"
#include "LevelEditor.h"

#define LOCTEXT_NAMESPACE "SHLODCompareWindow"

void SHLODCompareWindow::Construct(const FArguments& InArgs)
{
	HLODActors = InArgs._HLODActors;

	SourcePreviewScene = MakeShared<FPreviewScene>(FPreviewScene::ConstructionValues()
		.SetCreateDefaultLighting(true)
		.SetCreatePhysicsScene(false)
		.ShouldSimulatePhysics(false)
		.SetTransactional(false));

	HLODPreviewScene = MakeShared<FPreviewScene>(FPreviewScene::ConstructionValues()
		.SetCreateDefaultLighting(true)
		.SetCreatePhysicsScene(false)
		.ShouldSimulatePhysics(false)
		.SetTransactional(false));

	TSharedPtr<SHLODCompareWindow> ThisPtr = SharedThis(this);

	ChildSlot
	[
		SNew(SVerticalBox)

		// Labels row
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f, 2.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Left)
			.Padding(8.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("SourceActorsLabel", "Source Actors"))
				.Font(FCoreStyle::GetDefaultFontStyle("Bold", 11))
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Right)
			.Padding(8.0f, 0.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("HLODLabel", "HLOD"))
				.Font(FCoreStyle::GetDefaultFontStyle("Bold", 11))
			]
		]

		// Toolbar row: view mode selector (left) + Go to HLOD distance button (right)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f, 0.0f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(ViewModeToolbarContainer, SBox)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(8.0f, 0.0f)
			[
				SNew(SButton)
				.Text(LOCTEXT("GoToHLODDistance", "Go to HLOD Distance"))
				.ToolTipText(LOCTEXT("GoToHLODDistance_Tooltip", "Move camera to the distance at which the HLOD first becomes visible"))
				.OnClicked(this, &SHLODCompareWindow::OnGoToMinVisibleDistance)
			]
		]

		// Wipe overlay viewports
		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SNew(SOverlay)

			// Z=0: Source viewport (full area, receives all input)
			+ SOverlay::Slot()
			[
				SAssignNew(SourceViewport, SHLODCompareViewport)
				.PreviewScene(SourcePreviewScene)
				.CompareWindow(ThisPtr)
			]

			// Z=1: Clipped HLOD viewport (HitTestInvisible so input passes through)
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SAssignNew(ClipWrapper, SHLODClipWrapper)
				.WipePosition(TAttribute<float>::CreateLambda([this]() { return WipePosition; }))
				[
					SAssignNew(HLODViewport, SHLODCompareViewport)
					.PreviewScene(HLODPreviewScene)
					.CompareWindow(ThisPtr)
				]
			]

			// Z=2: Wipe handle via SCanvas
			+ SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(SCanvas)
				+ SCanvas::Slot()
				.Position(TAttribute<FVector2D>::CreateLambda([this]()
				{
					// Get canvas geometry to compute pixel position
					float Width = 1.0f;
					if (SourceViewport.IsValid())
					{
						Width = FMath::Max(SourceViewport->GetTickSpaceGeometry().GetLocalSize().X, 1.0f);
					}
					float HandleX = Width * WipePosition - 6.0f; // center 12px handle on wipe line
					return FVector2D(HandleX, 0.0);
				}))
				.Size(TAttribute<FVector2D>::CreateLambda([this]()
				{
					float Height = 1.0f;
					if (SourceViewport.IsValid())
					{
						Height = FMath::Max(SourceViewport->GetTickSpaceGeometry().GetLocalSize().Y, 1.0f);
					}
					return FVector2D(12.0, Height);
				}))
				[
					SNew(SHLODWipeHandle)
					.OnWipePositionChanged(FOnWipePositionChanged::CreateSP(this, &SHLODCompareWindow::OnWipePositionChanged))
				]
			]
		]

		// Status text (hidden by default)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f, 2.0f)
		[
			SAssignNew(StatusText, STextBlock)
			.ColorAndOpacity(FLinearColor::Yellow)
			.Visibility(EVisibility::Collapsed)
		]

		// Distance ruler (bottom of window)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f, 2.0f)
		[
			SNew(SHLODDistanceRuler)
			.SourceMinDrawDistance(TAttribute<float>::CreateSP(this, &SHLODCompareWindow::GetSourceMinDrawDistance))
			.HLODMinVisibleDistance(TAttribute<float>::CreateSP(this, &SHLODCompareWindow::GetHLODMinVisibleDistance))
			.CameraDistance(TAttribute<float>::CreateSP(this, &SHLODCompareWindow::GetCameraDistanceToBounds))
			.SourceZoneLabel(TAttribute<FString>::CreateSP(this, &SHLODCompareWindow::GetSourceZoneLabel))
			.HLODZoneLabel(TAttribute<FString>::CreateSP(this, &SHLODCompareWindow::GetHLODZoneLabel))
			.WorldToMeters(TAttribute<float>::CreateLambda([]()
			{
				UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr;
				return (World && World->GetWorldSettings()) ? World->GetWorldSettings()->WorldToMeters : 100.0f;
			}))
		]
	];

	// Register and bind keyboard shortcuts (must happen before building toolbar so command list is available)
	BindCommands();

	// Now that the source viewport is constructed, build and insert the view mode toolbar
	if (SourceViewport.IsValid())
	{
		ViewModeToolbarContainer->SetContent(SourceViewport->BuildExternalViewModeToolbar(CommandList));
	}

	// Set our command list as priority on the viewport so our shortcuts take precedence over standard ones
	if (SourceViewport.IsValid() && CommandList.IsValid())
	{
		SourceViewport->SetPriorityCommandList(CommandList);
	}

	// Wire up sibling relationship so HLOD client pulls buffer viz name from source before drawing
	if (SourceViewport.IsValid() && HLODViewport.IsValid())
	{
		TSharedPtr<FHLODCompareViewportClient> SourceClient = SourceViewport->GetCompareViewportClient();
		TSharedPtr<FHLODCompareViewportClient> HLODClient = HLODViewport->GetCompareViewportClient();
		if (SourceClient.IsValid() && HLODClient.IsValid())
		{
			HLODClient->SetSiblingClient(SourceClient);
		}
	}

	// Populate scenes
	bool bSourceOk = PopulateSourceScene();
	bool bHLODOk = PopulateHLODScene();

	if (!bSourceOk)
	{
		StatusText->SetText(LOCTEXT("SourceLoadFailed", "Warning: Failed to load source actors. Left side of wipe may be empty."));
		StatusText->SetVisibility(EVisibility::Visible);
	}
	else if (!bHLODOk)
	{
		StatusText->SetText(LOCTEXT("HLODLoadFailed", "Warning: HLOD actor has no visual components. Right side of wipe may be empty."));
		StatusText->SetVisibility(EVisibility::Visible);
	}

	CopyWorldLighting();
	FocusCamerasOnContent();

	// Defer keyboard focus to the source viewport until the window is visible
	if (SourceViewport.IsValid())
	{
		TWeakPtr<SHLODCompareViewport> WeakViewport = SourceViewport;
		RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateLambda(
			[WeakViewport](double, float) -> EActiveTimerReturnType
			{
				if (TSharedPtr<SHLODCompareViewport> Viewport = WeakViewport.Pin())
				{
					FSlateApplication::Get().SetKeyboardFocus(Viewport);
				}
				return EActiveTimerReturnType::Stop;
			}));
	}

	// Register for map changes so we clean up before the old world is GC'd
	if (FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		OnMapChangedHandle = LevelEditor.OnMapChanged().AddSP(this, &SHLODCompareWindow::OnMapChanged);
	}

	// Register for editor pre-exit so we clean up before OnEnginePreExit triggers
	// FPreviewScene::Uninitialize (which would fire OnWorldCleanup on HLOD actors
	// that try to access already-torn-down subsystems)
	OnEditorPreExitHandle = FEditorDelegates::OnEditorPreExit.AddSP(this, &SHLODCompareWindow::OnEditorPreExit);
}

SHLODCompareWindow::~SHLODCompareWindow()
{
	if (FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditor.OnMapChanged().Remove(OnMapChangedHandle);
	}
	FEditorDelegates::OnEditorPreExit.Remove(OnEditorPreExitHandle);

	// Remove components from preview scenes before destruction
	for (UActorComponent* Component : SourceSceneComponents)
	{
		if (Component)
		{
			SourcePreviewScene->RemoveComponent(Component);
		}
	}
	SourceSceneComponents.Empty();

	for (UActorComponent* Component : HLODSceneComponents)
	{
		if (Component)
		{
			HLODPreviewScene->RemoveComponent(Component);
		}
	}
	HLODSceneComponents.Empty();
}

void SHLODCompareWindow::RequestCloseWindow()
{
	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	if (ParentWindow.IsValid())
	{
		FSlateApplication::Get().DestroyWindowImmediately(ParentWindow.ToSharedRef());
	}
}

void SHLODCompareWindow::OnMapChanged(UWorld* World, EMapChangeType ChangeType)
{
	if (ChangeType == EMapChangeType::TearDownWorld)
	{
		RequestCloseWindow();
	}
}

void SHLODCompareWindow::OnEditorPreExit()
{
	RequestCloseWindow();
}

bool SHLODCompareWindow::PopulateSourceScene()
{
	UWorld* PreviewWorld = SourcePreviewScene->GetWorld();

	FScopedSlowTask SlowTask(static_cast<float>(HLODActors.Num()), LOCTEXT("LoadingSourceActors", "Loading HLOD source actors..."));
	SlowTask.MakeDialog();

	for (const TWeakObjectPtr<AWorldPartitionHLOD>& WeakHLODActor : HLODActors)
	{
		AWorldPartitionHLOD* HLODActorPtr = WeakHLODActor.Get();
		if (!HLODActorPtr)
		{
			continue;
		}

		SlowTask.EnterProgressFrame(1.0f, FText::Format(LOCTEXT("LoadingSourceActorsStep", "Loading source actors for {0}"), FText::FromString(HLODActorPtr->GetActorLabel())));

		UWorldPartitionHLODSourceActors* SourceActors = HLODActorPtr->GetSourceActors();
		if (!SourceActors)
		{
			continue;
		}

		// Load source actors directly into the preview scene's world
		bool bDirty = false;
		if (!SourceActors->LoadSourceActors(bDirty, PreviewWorld))
		{
			continue;
		}

		// Iterate loaded actors and add their primitive components to the preview scene
		for (TActorIterator<AActor> It(PreviewWorld); It; ++It)
		{
			AActor* Actor = *It;
			if (!SourceActors->IsHLODRelevant(Actor))
			{
				continue;
			}

			TArray<UPrimitiveComponent*> PrimitiveComponents;
			Actor->GetComponents<UPrimitiveComponent>(PrimitiveComponents);
			for (UPrimitiveComponent* PrimComp : PrimitiveComponents)
			{
				if (PrimComp && !SourceSceneComponents.Contains(PrimComp))
				{
					SourcePreviewScene->AddComponent(PrimComp, PrimComp->GetComponentTransform());
					SourceSceneComponents.Add(PrimComp);
				}
			}
		}
	}

	return SourceSceneComponents.Num() > 0;
}

bool SHLODCompareWindow::PopulateHLODScene()
{
	for (const TWeakObjectPtr<AWorldPartitionHLOD>& WeakHLODActor : HLODActors)
	{
		AWorldPartitionHLOD* HLODActorPtr = WeakHLODActor.Get();
		if (!HLODActorPtr)
		{
			continue;
		}

		const TArray<UActorComponent*>& InstanceComponents = HLODActorPtr->GetInstanceComponents();
		for (UActorComponent* Component : InstanceComponents)
		{
			USceneComponent* SceneComp = Cast<USceneComponent>(Component);
			if (!SceneComp)
			{
				continue;
			}

			// Duplicate the component into the HLOD preview scene's world so that the HLOD actor
			// is not affected and the preview scene takes ownership.
			USceneComponent* DuplicatedComp = DuplicateObject<USceneComponent>(SceneComp, HLODPreviewScene->GetWorld());
			if (DuplicatedComp)
			{
				HLODPreviewScene->AddComponent(DuplicatedComp, SceneComp->GetComponentTransform());
				HLODSceneComponents.Add(DuplicatedComp);
			}
		}
	}

	return HLODSceneComponents.Num() > 0;
}

void SHLODCompareWindow::OnCameraMoved(FHLODCompareViewportClient* Source)
{
	if (!SourceViewport.IsValid() || !HLODViewport.IsValid())
	{
		return;
	}

	TSharedPtr<FHLODCompareViewportClient> SourceClient = SourceViewport->GetCompareViewportClient();
	TSharedPtr<FHLODCompareViewportClient> HLODClient = HLODViewport->GetCompareViewportClient();

	if (!SourceClient.IsValid() || !HLODClient.IsValid())
	{
		return;
	}

	// Sync the sibling viewport to match the one the user is controlling
	if (Source == SourceClient.Get())
	{
		HLODClient->SyncCameraFrom(*SourceClient);
	}
	else if (Source == HLODClient.Get())
	{
		SourceClient->SyncCameraFrom(*HLODClient);
	}
}

void SHLODCompareWindow::OnViewModeChanged(FHLODCompareViewportClient* Source)
{
	if (!SourceViewport.IsValid() || !HLODViewport.IsValid())
	{
		return;
	}

	TSharedPtr<FHLODCompareViewportClient> SourceClient = SourceViewport->GetCompareViewportClient();
	TSharedPtr<FHLODCompareViewportClient> HLODClient = HLODViewport->GetCompareViewportClient();

	if (!SourceClient.IsValid() || !HLODClient.IsValid())
	{
		return;
	}

	// Sync view mode to the sibling viewport
	if (Source == SourceClient.Get())
	{
		HLODClient->SyncViewModeFrom(*SourceClient);
	}
	else if (Source == HLODClient.Get())
	{
		SourceClient->SyncViewModeFrom(*HLODClient);
	}

}

void SHLODCompareWindow::CopyWorldLighting()
{
	UWorld* EditorWorld = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr;
	if (!EditorWorld)
	{
		return;
	}

	auto ApplyToScene = [](FPreviewScene* Scene, UDirectionalLightComponent* WorldDirLight, USkyLightComponent* WorldSkyLight)
	{
		if (!Scene)
		{
			return;
		}

		if (WorldDirLight && Scene->DirectionalLight)
		{
			Scene->SetLightDirection(WorldDirLight->GetComponentRotation());
			Scene->SetLightBrightness(WorldDirLight->Intensity);
			Scene->SetLightColor(WorldDirLight->LightColor);
		}

		if (WorldSkyLight && Scene->SkyLight)
		{
			Scene->SetSkyBrightness(WorldSkyLight->Intensity);
			if (WorldSkyLight->SourceType == ESkyLightSourceType::SLS_SpecifiedCubemap && WorldSkyLight->Cubemap)
			{
				Scene->SetSkyCubemap(WorldSkyLight->Cubemap);
			}
			Scene->SkyLight->bLowerHemisphereIsBlack = WorldSkyLight->bLowerHemisphereIsBlack;
		}
	};

	// Find the first directional light and sky light in the editor world
	UDirectionalLightComponent* WorldDirLight = nullptr;
	USkyLightComponent* WorldSkyLight = nullptr;

	for (TActorIterator<AActor> It(EditorWorld); It; ++It)
	{
		if (!WorldDirLight)
		{
			WorldDirLight = It->FindComponentByClass<UDirectionalLightComponent>();
		}
		if (!WorldSkyLight)
		{
			WorldSkyLight = It->FindComponentByClass<USkyLightComponent>();
		}
		if (WorldDirLight && WorldSkyLight)
		{
			break;
		}
	}

	ApplyToScene(SourcePreviewScene.Get(), WorldDirLight, WorldSkyLight);
	ApplyToScene(HLODPreviewScene.Get(), WorldDirLight, WorldSkyLight);
}

void SHLODCompareWindow::FocusCamerasOnContent()
{
	// Compute bounding box from source actors only
	FBox SourceBounds(ForceInit);
	for (UActorComponent* Component : SourceSceneComponents)
	{
		if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Component))
		{
			SourceBounds += PrimComp->Bounds.GetBox();
		}
	}

	if (!SourceBounds.IsValid)
	{
		return;
	}

	const FVector Center = SourceBounds.GetCenter();

	// Start from the active editor viewport's camera
	FVector CameraLocation = Center - FVector(1, 0, 0) * 500.0f;
	FRotator CameraRotation = FRotator::ZeroRotator;

	if (GCurrentLevelEditingViewportClient)
	{
		CameraLocation = GCurrentLevelEditingViewportClient->GetViewLocation();
		CameraRotation = GCurrentLevelEditingViewportClient->GetViewRotation();
	}

	// Clamp distance: if camera is further than MinVisibleDistance from the bounds, bring it in.
	// Use the minimum MinVisibleDistance across all HLOD actors.
	float MinVisibleDistance = 0.0f;
	for (const TWeakObjectPtr<AWorldPartitionHLOD>& WeakHLODActor : HLODActors)
	{
		if (AWorldPartitionHLOD* HLODActorPtr = WeakHLODActor.Get())
		{
			float Dist = static_cast<float>(HLODActorPtr->GetMinVisibleDistance());
			if (MinVisibleDistance == 0.0f || (Dist > 0.0f && Dist < MinVisibleDistance))
			{
				MinVisibleDistance = Dist;
			}
		}
	}
	if (MinVisibleDistance > 0.0f)
	{
		const float DistanceToBounds = FMath::Sqrt(SourceBounds.ComputeSquaredDistanceToPoint(CameraLocation));
		const float BoundsRadius = SourceBounds.GetExtent().Size();
		const float DesiredDistance = BoundsRadius + MinVisibleDistance;
		const FVector ToCamera = CameraLocation - Center;
		if (ToCamera.Size() > DesiredDistance)
		{
			CameraLocation = Center + ToCamera.GetSafeNormal() * DesiredDistance;
		}
	}

	TSharedPtr<FHLODCompareViewportClient> SourceClient = SourceViewport.IsValid() ? SourceViewport->GetCompareViewportClient() : nullptr;
	TSharedPtr<FHLODCompareViewportClient> HLODClient = HLODViewport.IsValid() ? HLODViewport->GetCompareViewportClient() : nullptr;

	auto ApplyCamera = [&](const TSharedPtr<FHLODCompareViewportClient>& ViewportClient)
	{
		if (ViewportClient.IsValid())
		{
			ViewportClient->SetViewLocation(CameraLocation);
			ViewportClient->SetViewRotation(CameraRotation);
			ViewportClient->SetLookAtLocation(Center);
		}
	};

	ApplyCamera(SourceClient);
	ApplyCamera(HLODClient);
}

float SHLODCompareWindow::GetCameraDistanceToBounds() const
{
	TSharedPtr<FHLODCompareViewportClient> Client = SourceViewport.IsValid() ? SourceViewport->GetCompareViewportClient() : nullptr;
	if (!Client.IsValid())
	{
		return 0.0f;
	}

	FBox SourceBounds(ForceInit);
	for (UActorComponent* Component : SourceSceneComponents)
	{
		if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Component))
		{
			SourceBounds += PrimComp->Bounds.GetBox();
		}
	}

	if (!SourceBounds.IsValid)
	{
		return 0.0f;
	}

	const float BoundsRadius = SourceBounds.GetExtent().Size();
	const float DistToCenter = FVector::Dist(Client->GetViewLocation(), SourceBounds.GetCenter());
	return FMath::Max(DistToCenter - BoundsRadius, 0.0f);
}

float SHLODCompareWindow::GetSourceMinDrawDistance() const
{
	// If the source actors are HLODs (LODLevel > 0), find them in the preview world
	// and read their MinVisibleDistance directly.
	if (HLODActors.Num() > 0 && HLODActors[0].IsValid() && HLODActors[0]->GetLODLevel() > 0)
	{
		UWorld* PreviewWorld = SourcePreviewScene.IsValid() ? SourcePreviewScene->GetWorld() : nullptr;
		if (PreviewWorld)
		{
			float MaxMinVisible = 0.0f;
			for (TActorIterator<AWorldPartitionHLOD> It(PreviewWorld); It; ++It)
			{
				MaxMinVisible = FMath::Max(MaxMinVisible, static_cast<float>(It->GetMinVisibleDistance()));
			}
			if (MaxMinVisible > 0.0f)
			{
				return MaxMinVisible;
			}
		}
	}

	// For regular geometry, use the bounds radius as the reference point on the ruler
	FBox SourceBounds(ForceInit);
	for (UActorComponent* Component : SourceSceneComponents)
	{
		if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Component))
		{
			SourceBounds += PrimComp->Bounds.GetBox();
		}
	}
	if (SourceBounds.IsValid)
	{
		return SourceBounds.GetExtent().Size();
	}

	return 0.0f;
}

float SHLODCompareWindow::GetHLODMinVisibleDistance() const
{
	float MinDist = 0.0f;
	for (const TWeakObjectPtr<AWorldPartitionHLOD>& WeakHLODActor : HLODActors)
	{
		if (AWorldPartitionHLOD* HLODActorPtr = WeakHLODActor.Get())
		{
			float Dist = static_cast<float>(HLODActorPtr->GetMinVisibleDistance());
			if (MinDist == 0.0f || (Dist > 0.0f && Dist < MinDist))
			{
				MinDist = Dist;
			}
		}
	}
	return MinDist;
}

FString SHLODCompareWindow::GetSourceZoneLabel() const
{
	// If the HLOD is level N, the source is either "Actors" (N==0) or "HLOD(N-1)"
	if (HLODActors.Num() > 0 && HLODActors[0].IsValid())
	{
		uint32 LODLevel = HLODActors[0]->GetLODLevel();
		if (LODLevel > 0)
		{
			return FString::Printf(TEXT("HLOD%u"), LODLevel - 1);
		}
	}
	return TEXT("Actors");
}

FString SHLODCompareWindow::GetHLODZoneLabel() const
{
	UWorld* World = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr;
	float W2M = (World && World->GetWorldSettings()) ? World->GetWorldSettings()->WorldToMeters : 100.0f;

	if (HLODActors.Num() > 0 && HLODActors[0].IsValid())
	{
		uint32 LODLevel = HLODActors[0]->GetLODLevel();
		float Dist = GetHLODMinVisibleDistance();
		return FString::Printf(TEXT("HLOD%u (%.0fm)"), LODLevel, Dist / W2M);
	}
	return TEXT("HLOD");
}

void SHLODCompareWindow::SetCompareViewMode(EViewModeIndex ViewMode, FName BufferVizMode)
{
	TSharedPtr<FHLODCompareViewportClient> SourceClient = SourceViewport.IsValid() ? SourceViewport->GetCompareViewportClient() : nullptr;
	TSharedPtr<FHLODCompareViewportClient> HLODClient = HLODViewport.IsValid() ? HLODViewport->GetCompareViewportClient() : nullptr;

	if (SourceClient.IsValid())
	{
		if (ViewMode == VMI_VisualizeBuffer)
		{
			SourceClient->ChangeBufferVisualizationMode(BufferVizMode);
		}
		else
		{
			SourceClient->SetViewMode(ViewMode);
		}
	}

	if (HLODClient.IsValid())
	{
		if (ViewMode == VMI_VisualizeBuffer)
		{
			HLODClient->ChangeBufferVisualizationMode(BufferVizMode);
		}
		else
		{
			HLODClient->SetViewMode(ViewMode);
		}
	}
}

void SHLODCompareWindow::BindCommands()
{
	FHLODCompareCommands::Register();
	const FHLODCompareCommands& Commands = FHLODCompareCommands::Get();

	CommandList = MakeShared<FUICommandList>();

	auto IsViewMode = [this](EViewModeIndex ViewMode, FName BufferVizMode) -> bool
	{
		TSharedPtr<FHLODCompareViewportClient> Client = SourceViewport.IsValid() ? SourceViewport->GetCompareViewportClient() : nullptr;
		if (!Client.IsValid())
		{
			return false;
		}
		if (ViewMode == VMI_VisualizeBuffer)
		{
			return Client->GetViewMode() == VMI_VisualizeBuffer && Client->CurrentBufferVisualizationMode == BufferVizMode;
		}
		return Client->GetViewMode() == ViewMode;
	};

	auto MapViewMode = [&](const TSharedPtr<FUICommandInfo>& Command, EViewModeIndex ViewMode, FName BufferVizMode = NAME_None)
	{
		CommandList->MapAction(Command,
			FExecuteAction::CreateSP(this, &SHLODCompareWindow::SetCompareViewMode, ViewMode, BufferVizMode),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this, ViewMode, BufferVizMode, IsViewMode]() { return IsViewMode(ViewMode, BufferVizMode); }));
	};

	MapViewMode(Commands.ViewModeLit, VMI_Lit);
	MapViewMode(Commands.ViewModeWireframe, VMI_BrushWireframe);
	MapViewMode(Commands.ViewModeBaseColor, VMI_VisualizeBuffer, FName(TEXT("BaseColor")));
	MapViewMode(Commands.ViewModeMetallic, VMI_VisualizeBuffer, FName(TEXT("Metallic")));
	MapViewMode(Commands.ViewModeRoughness, VMI_VisualizeBuffer, FName(TEXT("Roughness")));
	MapViewMode(Commands.ViewModeSpecular, VMI_VisualizeBuffer, FName(TEXT("Specular")));
	MapViewMode(Commands.ViewModeWorldNormal, VMI_VisualizeBuffer, FName(TEXT("WorldNormal")));
	CommandList->MapAction(Commands.GoToHLODDistance,
		FExecuteAction::CreateLambda([this]() { OnGoToMinVisibleDistance(); }));
}

FReply SHLODCompareWindow::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	// Let the command list handle registered shortcuts
	if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	// Swallow other Alt+number combos so they don't reach the viewport
	if (InKeyEvent.IsAltDown())
	{
		const FKey Key = InKeyEvent.GetKey();
		if (Key == EKeys::Eight || Key == EKeys::Nine || Key == EKeys::Zero)
		{
			return FReply::Handled();
		}
	}

	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

void SHLODCompareWindow::OnWipePositionChanged(float NewPosition)
{
	WipePosition = FMath::Clamp(NewPosition, 0.0f, 1.0f);
}

FReply SHLODCompareWindow::OnGoToMinVisibleDistance()
{
	// Compute bounds from source actors (used for distance computation)
	FBox SourceBounds(ForceInit);
	for (UActorComponent* Component : SourceSceneComponents)
	{
		if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Component))
		{
			SourceBounds += PrimComp->Bounds.GetBox();
		}
	}

	if (!SourceBounds.IsValid)
	{
		return FReply::Handled();
	}

	// Compute HLOD bounds center (used as look-at target)
	FBox HLODBounds(ForceInit);
	for (UActorComponent* Component : HLODSceneComponents)
	{
		if (UPrimitiveComponent* PrimComp = Cast<UPrimitiveComponent>(Component))
		{
			HLODBounds += PrimComp->Bounds.GetBox();
		}
	}

	const FVector LookAtCenter = HLODBounds.IsValid ? HLODBounds.GetCenter() : SourceBounds.GetCenter();
	const FVector SourceCenter = SourceBounds.GetCenter();

	// Find the minimum MinVisibleDistance across all HLOD actors
	float MinVisibleDistance = 0.0f;
	for (const TWeakObjectPtr<AWorldPartitionHLOD>& WeakHLODActor : HLODActors)
	{
		if (AWorldPartitionHLOD* HLODActorPtr = WeakHLODActor.Get())
		{
			float Dist = static_cast<float>(HLODActorPtr->GetMinVisibleDistance());
			if (MinVisibleDistance == 0.0f || (Dist > 0.0f && Dist < MinVisibleDistance))
			{
				MinVisibleDistance = Dist;
			}
		}
	}

	if (MinVisibleDistance <= 0.0f)
	{
		return FReply::Handled();
	}

	TSharedPtr<FHLODCompareViewportClient> SourceClient = SourceViewport.IsValid() ? SourceViewport->GetCompareViewportClient() : nullptr;
	TSharedPtr<FHLODCompareViewportClient> HLODClient = HLODViewport.IsValid() ? HLODViewport->GetCompareViewportClient() : nullptr;

	if (!SourceClient.IsValid())
	{
		return FReply::Handled();
	}

	// Move camera along its current view direction to MinVisibleDistance from the source box surface,
	// looking at the HLOD bounds center
	const FVector CurrentLocation = SourceClient->GetViewLocation();
	const FVector Direction = (CurrentLocation - SourceCenter).GetSafeNormal();
	const float BoundsRadius = SourceBounds.GetExtent().Size();
	const FVector NewLocation = SourceCenter + Direction * (BoundsRadius + MinVisibleDistance);
	const FRotator NewRotation = (LookAtCenter - NewLocation).Rotation();

	SourceClient->SetViewLocation(NewLocation);
	SourceClient->SetViewRotation(NewRotation);
	SourceClient->SetLookAtLocation(LookAtCenter);

	if (HLODClient.IsValid())
	{
		HLODClient->SetViewLocation(NewLocation);
		HLODClient->SetViewRotation(NewRotation);
		HLODClient->SetLookAtLocation(LookAtCenter);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
