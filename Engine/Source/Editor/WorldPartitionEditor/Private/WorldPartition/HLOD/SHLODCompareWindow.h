// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Engine/EngineBaseTypes.h"
#include "Framework/Commands/UICommandList.h"

enum class EMapChangeType : uint8;

class AWorldPartitionHLOD;
class FPreviewScene;
class SHLODCompareViewport;
class FHLODCompareViewportClient;
class SHLODClipWrapper;
class SBox;
class STextBlock;
class UActorComponent;

class SHLODCompareWindow : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SHLODCompareWindow) {}
		SLATE_ARGUMENT(TArray<TWeakObjectPtr<AWorldPartitionHLOD>>, HLODActors)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	~SHLODCompareWindow();

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual bool SupportsKeyboardFocus() const override { return true; }

	void OnCameraMoved(FHLODCompareViewportClient* Source);
	void OnViewModeChanged(FHLODCompareViewportClient* Source);

private:
	void BindCommands();
	void SetCompareViewMode(EViewModeIndex ViewMode, FName BufferVizMode = NAME_None);
	bool PopulateSourceScene();
	bool PopulateHLODScene();
	void FocusCamerasOnContent();
	void CopyWorldLighting();
	void OnWipePositionChanged(float NewPosition);
	FReply OnGoToMinVisibleDistance();
	void RequestCloseWindow();
	void OnMapChanged(UWorld* World, EMapChangeType ChangeType);
	void OnEditorPreExit();

	TArray<TWeakObjectPtr<AWorldPartitionHLOD>> HLODActors;

	TSharedPtr<FPreviewScene> SourcePreviewScene;
	TSharedPtr<FPreviewScene> HLODPreviewScene;

	TSharedPtr<SHLODCompareViewport> SourceViewport;
	TSharedPtr<SHLODCompareViewport> HLODViewport;

	TArray<UActorComponent*> SourceSceneComponents;
	TArray<UActorComponent*> HLODSceneComponents;

	float GetCameraDistanceToBounds() const;
	float GetSourceMinDrawDistance() const;
	float GetHLODMinVisibleDistance() const;
	FString GetSourceZoneLabel() const;
	FString GetHLODZoneLabel() const;

	TSharedPtr<STextBlock> StatusText;

	TSharedPtr<FUICommandList> CommandList;

	// Wipe overlay
	float WipePosition = 0.5f;
	TSharedPtr<SHLODClipWrapper> ClipWrapper;
	TSharedPtr<SBox> ViewModeToolbarContainer;

	FDelegateHandle OnMapChangedHandle;
	FDelegateHandle OnEditorPreExitHandle;
};
