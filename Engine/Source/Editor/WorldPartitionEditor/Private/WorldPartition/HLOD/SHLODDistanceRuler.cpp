// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/SHLODDistanceRuler.h"
#include "Rendering/DrawElements.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "SHLODDistanceRuler"

static constexpr float RulerHeight = 40.0f;
static constexpr float CameraIconSize = 12.0f;
static constexpr float CameraRowY = 2.0f;
static constexpr float TrackY = CameraRowY + CameraIconSize + 3.0f;
static constexpr float TrackHeight = 3.0f;
static constexpr float MarkerHeight = 10.0f;
static constexpr float HorizontalPadding = 16.0f;

// Colors: green = correct range (source visible), grey = too close
static const FLinearColor CorrectRangeColor(0.1f, 0.5f, 0.1f, 0.8f);
static const FLinearColor CorrectRangeMarkerColor(0.4f, 0.8f, 0.4f, 1.0f);
static const FLinearColor TooCloseColor(0.25f, 0.25f, 0.25f, 0.8f);
static const FLinearColor TooCloseMarkerColor(0.5f, 0.5f, 0.5f, 1.0f);

void SHLODDistanceRuler::Construct(const FArguments& InArgs)
{
	SourceMinDrawDistance = InArgs._SourceMinDrawDistance;
	HLODMinVisibleDistance = InArgs._HLODMinVisibleDistance;
	CameraDistance = InArgs._CameraDistance;
	SourceZoneLabel = InArgs._SourceZoneLabel;
	HLODZoneLabel = InArgs._HLODZoneLabel;
	WorldToMeters = InArgs._WorldToMeters;
}

FVector2D SHLODDistanceRuler::ComputeDesiredSize(float) const
{
	return FVector2D(100.0f, RulerHeight);
}

int32 SHLODDistanceRuler::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const float Width = AllottedGeometry.GetLocalSize().X;
	const float TrackLeft = HorizontalPadding;
	const float TrackRight = Width - HorizontalPadding;
	const float TrackWidth = TrackRight - TrackLeft;

	if (TrackWidth <= 0.0f)
	{
		return LayerId;
	}

	const float SourceDist = SourceMinDrawDistance.Get();
	const float HLODDist = HLODMinVisibleDistance.Get();
	const float CamDist = CameraDistance.Get();

	// The ruler range: show from 0 to a bit beyond HLODMinVisibleDistance
	const float MaxRange = FMath::Max(HLODDist * 1.3f, FMath::Max(SourceDist, 1.0f) * 3.0f);

	auto DistToX = [&](float Dist) -> float
	{
		return TrackLeft + FMath::Clamp(Dist / MaxRange, 0.0f, 1.0f) * TrackWidth;
	};

	const FSlateBrush* WhiteBrush = FCoreStyle::Get().GetBrush("WhiteBrush");
	const FSlateFontInfo Font = FCoreStyle::GetDefaultFontStyle("Regular", 8);
	const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	// Draw the track background
	FSlateDrawElement::MakeBox(
		OutDrawElements, LayerId,
		AllottedGeometry.ToPaintGeometry(FVector2D(TrackWidth, TrackHeight), FSlateLayoutTransform(FVector2D(TrackLeft, TrackY))),
		WhiteBrush, ESlateDrawEffect::None, FLinearColor(0.15f, 0.15f, 0.15f, 1.0f));

	// Source zone (0 to HLODMinVisibleDistance) - grey, too close for meaningful HLOD comparison
	{
		float ZoneRight = DistToX(HLODDist);
		float ZoneWidth = ZoneRight - TrackLeft;
		if (ZoneWidth > 0.0f)
		{
			FSlateDrawElement::MakeBox(
				OutDrawElements, LayerId + 1,
				AllottedGeometry.ToPaintGeometry(FVector2D(ZoneWidth, TrackHeight), FSlateLayoutTransform(FVector2D(TrackLeft, TrackY))),
				WhiteBrush, ESlateDrawEffect::None, TooCloseColor);
		}
	}

	// HLOD zone (HLODMinVisibleDistance onward) - green, the distance at which the HLOD is seen in-game
	if (HLODDist > 0.0f)
	{
		float ZoneLeft = DistToX(HLODDist);
		float ZoneWidth = TrackRight - ZoneLeft;
		if (ZoneWidth > 0.0f)
		{
			FSlateDrawElement::MakeBox(
				OutDrawElements, LayerId + 1,
				AllottedGeometry.ToPaintGeometry(FVector2D(ZoneWidth, TrackHeight), FSlateLayoutTransform(FVector2D(ZoneLeft, TrackY))),
				WhiteBrush, ESlateDrawEffect::None, CorrectRangeColor);
		}
	}

	// Camera indicator — above the track, colored by zone
	{
		float CamX = DistToX(CamDist);
		const FLinearColor CamColor = (CamDist >= HLODDist) ? CorrectRangeMarkerColor : TooCloseMarkerColor;

		// Camera icon
		const FSlateBrush* CameraBrush = FAppStyle::Get().GetBrush("ClassIcon.CameraComponent");
		float IconX = CamX - CameraIconSize * 0.5f;
		FSlateDrawElement::MakeBox(
			OutDrawElements, LayerId + 3,
			AllottedGeometry.ToPaintGeometry(FVector2D(CameraIconSize, CameraIconSize), FSlateLayoutTransform(FVector2D(IconX, CameraRowY))),
			CameraBrush, ESlateDrawEffect::None, CamColor);

		// Camera distance label in meters
		const float W2M = FMath::Max(WorldToMeters.Get(), 1.0f);
		FString CamLabel = FString::Printf(TEXT("(%.1fm)"), CamDist / W2M);
		FVector2D LabelSize = FontMeasure->Measure(CamLabel, Font);
		float LabelX = FMath::Clamp(CamX + CameraIconSize * 0.5f + 3.0f, TrackLeft, TrackRight - LabelSize.X);
		FSlateDrawElement::MakeText(
			OutDrawElements, LayerId + 3,
			AllottedGeometry.ToPaintGeometry(LabelSize, FSlateLayoutTransform(FVector2D(LabelX, CameraRowY + (CameraIconSize - LabelSize.Y) * 0.5f))),
			CamLabel, Font, ESlateDrawEffect::None, CamColor);
	}

	// Markers and labels — below the track
	const float MarkerTopY = TrackY + TrackHeight;
	const float LabelY = MarkerTopY + MarkerHeight + 1.0f;

	// Source MinDrawDistance marker (grey)
	if (SourceDist > 0.0f)
	{
		float MarkerX = DistToX(SourceDist);
		FSlateDrawElement::MakeBox(
			OutDrawElements, LayerId + 2,
			AllottedGeometry.ToPaintGeometry(FVector2D(1.0f, MarkerHeight), FSlateLayoutTransform(FVector2D(MarkerX, MarkerTopY))),
			WhiteBrush, ESlateDrawEffect::None, TooCloseMarkerColor);
	}

	// HLOD MinVisibleDistance marker (green)
	if (HLODDist > 0.0f)
	{
		float MarkerX = DistToX(HLODDist);
		FSlateDrawElement::MakeBox(
			OutDrawElements, LayerId + 2,
			AllottedGeometry.ToPaintGeometry(FVector2D(1.0f, MarkerHeight), FSlateLayoutTransform(FVector2D(MarkerX, MarkerTopY))),
			WhiteBrush, ESlateDrawEffect::None, CorrectRangeMarkerColor);
	}

	// Source zone label — at SourceMinDrawDistance marker position (grey)
	{
		FString Label = SourceZoneLabel.Get();
		if (!Label.IsEmpty())
		{
			float MarkerX = DistToX(SourceDist);
			FVector2D LabelSize = FontMeasure->Measure(Label, Font);
			float LabelX = FMath::Clamp(MarkerX - LabelSize.X * 0.5f, TrackLeft, TrackRight - LabelSize.X);
			FSlateDrawElement::MakeText(
				OutDrawElements, LayerId + 2,
				AllottedGeometry.ToPaintGeometry(LabelSize, FSlateLayoutTransform(FVector2D(LabelX, LabelY))),
				Label, Font, ESlateDrawEffect::None, TooCloseMarkerColor);
		}
	}

	// HLOD zone label — at HLOD MinVisibleDistance marker position (green)
	if (HLODDist > 0.0f)
	{
		FString Label = HLODZoneLabel.Get();
		if (!Label.IsEmpty())
		{
			float MarkerX = DistToX(HLODDist);
			FVector2D LabelSize = FontMeasure->Measure(Label, Font);
			float LabelX = FMath::Clamp(MarkerX - LabelSize.X * 0.5f, TrackLeft, TrackRight - LabelSize.X);
			FSlateDrawElement::MakeText(
				OutDrawElements, LayerId + 2,
				AllottedGeometry.ToPaintGeometry(LabelSize, FSlateLayoutTransform(FVector2D(LabelX, LabelY))),
				Label, Font, ESlateDrawEffect::None, CorrectRangeMarkerColor);
		}
	}

	return LayerId + 4;
}

#undef LOCTEXT_NAMESPACE
