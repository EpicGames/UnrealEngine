// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SLeafWidget.h"

/**
 * A horizontal ruler showing camera distance relative to source actors and HLOD transition.
 *
 * Layout:
 *   |--- Source visible ---|- gap -|--- HLOD visible ---|
 *   0    SourceMinDraw     ^       HLODMinVisible       far
 *        "Actors"          camera  "HLOD1 (128.0m)"
 */
class SHLODDistanceRuler : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SHLODDistanceRuler)
		: _SourceMinDrawDistance(0.0f)
		, _HLODMinVisibleDistance(0.0f)
		, _CameraDistance(0.0f)
	{}
		SLATE_ATTRIBUTE(float, SourceMinDrawDistance)
		SLATE_ATTRIBUTE(float, HLODMinVisibleDistance)
		SLATE_ATTRIBUTE(float, CameraDistance)
		SLATE_ATTRIBUTE(FString, SourceZoneLabel)
		SLATE_ATTRIBUTE(FString, HLODZoneLabel)
		SLATE_ATTRIBUTE(float, WorldToMeters)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	// SWidget interface
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FVector2D ComputeDesiredSize(float) const override;

private:
	TAttribute<float> SourceMinDrawDistance;
	TAttribute<float> HLODMinVisibleDistance;
	TAttribute<float> CameraDistance;
	TAttribute<FString> SourceZoneLabel;
	TAttribute<FString> HLODZoneLabel;
	TAttribute<float> WorldToMeters;
};
