// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/HLOD/SHLODWipeHandle.h"
#include "Rendering/DrawElements.h"
#include "Styling/CoreStyle.h"

#define LOCTEXT_NAMESPACE "SHLODWipeHandle"

static constexpr float WipeHandleHitWidth = 12.0f;
static constexpr float WipeLineVisualWidth = 2.0f;

void SHLODWipeHandle::Construct(const FArguments& InArgs)
{
	OnWipePositionChanged = InArgs._OnWipePositionChanged;
}

int32 SHLODWipeHandle::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const FVector2D LocalSize = AllottedGeometry.GetLocalSize();

	// Draw a 2px vertical line centered in the hit area
	const float LineOffset = (LocalSize.X - WipeLineVisualWidth) * 0.5f;

	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(FVector2D(WipeLineVisualWidth, LocalSize.Y), FSlateLayoutTransform(FVector2D(LineOffset, 0.0f))),
		FCoreStyle::Get().GetBrush("WhiteBrush"),
		ESlateDrawEffect::None,
		FLinearColor(1.0f, 1.0f, 1.0f, 0.7f)
	);

	return LayerId + 1;
}

FVector2D SHLODWipeHandle::ComputeDesiredSize(float) const
{
	return FVector2D(WipeHandleHitWidth, 1.0f);
}

FReply SHLODWipeHandle::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		bDragging = true;
		return FReply::Handled().CaptureMouse(SharedThis(this));
	}
	return FReply::Unhandled();
}

FReply SHLODWipeHandle::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && bDragging)
	{
		bDragging = false;
		return FReply::Handled().ReleaseMouseCapture();
	}
	return FReply::Unhandled();
}

FReply SHLODWipeHandle::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bDragging)
	{
		// Find the parent SCanvas (or whatever contains us) to compute relative position
		// The wipe handle is placed inside an SCanvas that fills the entire overlay area.
		// We need to compute the wipe position relative to that parent canvas.
		TSharedPtr<SWidget> ParentWidget = GetParentWidget();
		if (ParentWidget.IsValid())
		{
			FGeometry ParentGeometry = ParentWidget->GetTickSpaceGeometry();
			FVector2D LocalMousePos = ParentGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
			float ParentWidth = ParentGeometry.GetLocalSize().X;

			if (ParentWidth > 0.0f)
			{
				float NewPosition = FMath::Clamp(LocalMousePos.X / ParentWidth, 0.0f, 1.0f);
				OnWipePositionChanged.ExecuteIfBound(NewPosition);
			}
		}

		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FCursorReply SHLODWipeHandle::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
}

#undef LOCTEXT_NAMESPACE
