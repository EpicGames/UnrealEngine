// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SLeafWidget.h"

DECLARE_DELEGATE_OneParam(FOnWipePositionChanged, float);

/**
 * A draggable vertical wipe handle for the HLOD compare overlay.
 * Draws a thin vertical line and handles mouse drag to update wipe position.
 */
class SHLODWipeHandle : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SHLODWipeHandle) {}
		SLATE_EVENT(FOnWipePositionChanged, OnWipePositionChanged)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	// SWidget interface
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;

private:
	FOnWipePositionChanged OnWipePositionChanged;
	bool bDragging = false;
};
