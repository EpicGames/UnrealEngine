// Copyright Epic Games, Inc. All Rights Reserved.
#include "SWorldPartitionBuildHLODsDialog.h"

#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "WorldPartitionBuildHLODsDialog"

const FVector2D SWorldPartitionBuildHLODsDialog::DEFAULT_WINDOW_SIZE = FVector2D(600, 200);

void SWorldPartitionBuildHLODsDialog::Construct(const FArguments& InArgs)
{
	ParentWindowPtr = InArgs._ParentWindow.Get();
	Result = DialogResult::Cancel;

	this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.MaxHeight(200.0f)
				[
					// Options here
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(20, 20, 20, 20)
					[
						SNew(STextBlock)
						.AutoWrapText(true)
						.Text(LOCTEXT("BuildHLODDescription", "Select the operation you wish to perform. Note that this will require the current map be saved & unloaded. It will be reopened once the operation is completed."))
					]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.f)
					[
						SNew(SSpacer)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.f)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.OnClicked(this, &SWorldPartitionBuildHLODsDialog::OnBuildClicked)
						.Text(LOCTEXT("BuildButton", "Build HLODs"))
					]
					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.f)
						[
							SNew(SButton)
							.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.OnClicked(this, &SWorldPartitionBuildHLODsDialog::OnDeleteClicked)
						.Text(LOCTEXT("DeleteButton", "Delete HLODs"))
						]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.f)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.OnClicked(this, &SWorldPartitionBuildHLODsDialog::OnCancelClicked)
						.Text(LOCTEXT("CancelButton", "Cancel"))
					]
				]
			]
		];
}

FReply SWorldPartitionBuildHLODsDialog::OnBuildClicked()
{
	Result = DialogResult::BuildHLODs;
	ParentWindowPtr.Pin()->RequestDestroyWindow();
	return FReply::Handled();
}


FReply SWorldPartitionBuildHLODsDialog::OnDeleteClicked()
{
	Result = DialogResult::DeleteHLODs;
	ParentWindowPtr.Pin()->RequestDestroyWindow();
	return FReply::Handled();
}

FReply SWorldPartitionBuildHLODsDialog::OnCancelClicked()
{
	Result = DialogResult::Cancel;
	ParentWindowPtr.Pin()->RequestDestroyWindow();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
