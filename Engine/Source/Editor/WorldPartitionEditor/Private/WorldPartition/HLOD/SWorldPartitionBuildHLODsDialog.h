// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Input/Reply.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SWindow;

//////////////////////////////////////////////////////////////////////////
// SWorldPartitionBuildHLODsDialog

class SWorldPartitionBuildHLODsDialog : public SCompoundWidget
{
public:
	enum DialogResult
	{
		BuildHLODs,
		DeleteHLODs,
		Cancel
	};

	SLATE_BEGIN_ARGS(SWorldPartitionBuildHLODsDialog) {}
		/** A pointer to the parent window */
		SLATE_ATTRIBUTE(TSharedPtr<SWindow>, ParentWindow)
	SLATE_END_ARGS()

	~SWorldPartitionBuildHLODsDialog() = default;

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel			The UI logic not specific to slate
	 */
	void Construct(const FArguments& InArgs);

	static const FVector2D DEFAULT_WINDOW_SIZE;

	DialogResult GetDialogResult() const { return Result; }
	
private:
	FReply OnBuildClicked();
	FReply OnDeleteClicked();
	FReply OnCancelClicked();
		
	/** Pointer to the parent window, so we know to destroy it when done */
	TWeakPtr<SWindow> ParentWindowPtr;
		
	/** Dialog Result */
	DialogResult Result;
};
