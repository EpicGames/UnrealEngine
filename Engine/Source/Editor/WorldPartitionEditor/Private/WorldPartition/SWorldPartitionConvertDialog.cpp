// Copyright Epic Games, Inc. All Rights Reserved.
#include "SWorldPartitionConvertDialog.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Input/SButton.h"
#include "WorldPartition/WorldPartitionConvertOptions.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "WorldPartitionConvertDialog"

const FVector2D SWorldPartitionConvertDialog::DEFAULT_WINDOW_SIZE = FVector2D(600, 350);

void SWorldPartitionConvertDialog::Construct(const FArguments& InArgs)
{
	ParentWindowPtr = InArgs._ParentWindow.Get();
	ConvertOptions = InArgs._ConvertOptions.Get();
	bClickedOk = false;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bAllowMultipleTopLevelObjects = false;

	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	
	// Set provided objects on SDetailsView
	DetailsView->SetObject(ConvertOptions.Get(), true);

	this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.MaxHeight(500.0f)
				[
					DetailsView->AsShared()
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.FillWidth(1.f)
					[
						SNew(SSpacer)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.f)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.IsEnabled(this, &SWorldPartitionConvertDialog::IsOkEnabled)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.OnClicked(this, &SWorldPartitionConvertDialog::OnOkClicked)
						.Text(LOCTEXT("OkButton", "Ok"))
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(2.f)
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.OnClicked(this, &SWorldPartitionConvertDialog::OnCancelClicked)
						.Text(LOCTEXT("CancelButton", "Cancel"))
					]
				]
			]
		];
}

bool SWorldPartitionConvertDialog::IsOkEnabled() const
{
	return ConvertOptions->CommandletClass != nullptr;
}

FReply SWorldPartitionConvertDialog::OnOkClicked()
{
	bClickedOk = true;
	ParentWindowPtr.Pin()->RequestDestroyWindow();
	return FReply::Handled();
}

FReply SWorldPartitionConvertDialog::OnCancelClicked()
{
	bClickedOk = false;
	ParentWindowPtr.Pin()->RequestDestroyWindow();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
