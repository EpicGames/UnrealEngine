// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Types/SlateEnums.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

//////////////////////////////////////////////////////////////////////////
// SWorldPartitionConvertDialog
class UWorldPartitionConvertOptions;

class SWorldPartitionConvertDialog : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SWorldPartitionConvertDialog) {}
		/** A pointer to the parent window */
		SLATE_ATTRIBUTE(TSharedPtr<SWindow>, ParentWindow)
		SLATE_ATTRIBUTE(UWorldPartitionConvertOptions*, ConvertOptions)
	SLATE_END_ARGS()

	~SWorldPartitionConvertDialog() = default;

	/**
	 * Construct this widget.  Called by the SNew() Slate macro.
	 *
	 * @param	InArgs				Declaration used by the SNew() macro to construct this widget
	 * @param	InViewModel			The UI logic not specific to slate
	 */
	void Construct(const FArguments& InArgs);

	static const FVector2D DEFAULT_WINDOW_SIZE;

	bool ClickedOk() const { return bClickedOk; }
	
private:
	FReply OnOkClicked();
	bool IsOkEnabled() const;

	FReply OnCancelClicked();
		
	/** Pointer to the parent window, so we know to destroy it when done */
	TWeakPtr<SWindow> ParentWindowPtr;
	TWeakObjectPtr<UWorldPartitionConvertOptions> ConvertOptions;

	/** Detailsview used to display SettingsObjects, and allowing user to change options */
	TSharedPtr<class IDetailsView> DetailsView;
		
	/** Dialog Result */
	bool bClickedOk;
};
