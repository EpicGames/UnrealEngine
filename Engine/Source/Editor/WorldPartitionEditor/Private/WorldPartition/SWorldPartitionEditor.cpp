// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/SWorldPartitionEditor.h"
#include "WorldPartition/SWorldPartitionEditorGrid.h"
#include "GameFramework/WorldSettings.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldBrowserModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Images/SImage.h"
#include "Styling/AppStyle.h"
#include "WorldPartition/IWorldPartitionEditorModule.h"

#define LOCTEXT_NAMESPACE "WorldPartitionEditor"

void SWorldPartitionEditor::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SAssignNew(ContentParent, SBorder)
		.BorderImage(FAppStyle::GetBrush(TEXT("ToolPanel.GroupBorder")))
	];

	OnBrowseWorld(InArgs._InWorld);

	FWorldBrowserModule& WorldBrowserModule = FModuleManager::LoadModuleChecked<FWorldBrowserModule>("WorldBrowser");
	WorldBrowserModule.OnBrowseWorld.AddSP(this, &SWorldPartitionEditor::OnBrowseWorld);
	UWorldPartition::WorldPartitionChangedEvent.AddSP(this, &SWorldPartitionEditor::OnBrowseWorld);
	
	IWorldPartitionEditorModule::Get().OnWorldPartitionCreated().AddSP(this, &SWorldPartitionEditor::OnBrowseWorld);
}

SWorldPartitionEditor::~SWorldPartitionEditor()
{
	FWorldBrowserModule& WorldBrowserModule = FModuleManager::GetModuleChecked<FWorldBrowserModule>("WorldBrowser");
	WorldBrowserModule.OnBrowseWorld.RemoveAll(this);
	UWorldPartition::WorldPartitionChangedEvent.RemoveAll(this);
	
	IWorldPartitionEditorModule::Get().OnWorldPartitionCreated().RemoveAll(this);

	if (World.IsValid())
	{
		if (UWorldPartition* WorldPartition = World->GetWorldPartition())
		{
			check(WorldPartition->GetWorld() == World);
			if (WorldPartition->WorldPartitionEditor)
			{
				check(WorldPartition->WorldPartitionEditor == this);
				WorldPartition->WorldPartitionEditor = nullptr;
			}
		}
	}
}

void SWorldPartitionEditor::Refresh()
{
	GridView->Refresh();
}

void SWorldPartitionEditor::Reconstruct()
{
	ContentParent->SetContent(ConstructContentWidget());
}

void SWorldPartitionEditor::FocusBox(const FBox& Box) const
{
	GridView->FocusBox(Box);
}

void SWorldPartitionEditor::OnBrowseWorld(UWorld* InWorld)
{
	World = InWorld;
	Reconstruct();
}

TSharedRef<SWidget> SWorldPartitionEditor::ConstructContentWidget()
{
	FName EditorName = NAME_None;
	if (World.IsValid())
	{
		if (UWorldPartition* WorldPartition = World->GetWorldPartition())
		{
			check(WorldPartition->GetWorld() == World);
			EditorName = WorldPartition->GetWorldPartitionEditorName();
			WorldPartition->WorldPartitionEditor = this;
		}
	}

	SWorldPartitionEditorGrid::PartitionEditorGridCreateInstanceFunc PartitionEditorGridCreateInstanceFunc = SWorldPartitionEditorGrid::GetPartitionEditorGridCreateInstanceFunc(EditorName);

	TSharedRef<SWidget> Result = 
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.FillHeight(1)
		[
			SNew(SOverlay)

			// Grid view
			+SOverlay::Slot()
			[
				PartitionEditorGridCreateInstanceFunc(GridView, World.Get())
			]

			// Grid view top status bar
			+SOverlay::Slot()
			.VAlign(VAlign_Top)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
					]
				]
			]

			// Grid view bottom status bar
			+SOverlay::Slot()
			.VAlign(VAlign_Bottom)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
					]
				]
			]
		];

	Refresh();
	return Result;
}

#undef LOCTEXT_NAMESPACE