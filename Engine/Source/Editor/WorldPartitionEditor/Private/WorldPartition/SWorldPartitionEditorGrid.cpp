// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/SWorldPartitionEditorGrid.h"
#include "WorldPartition/WorldPartition.h"
#include "GameFramework/WorldSettings.h"
#include "GameFramework/PlayerController.h"
#include "ISceneOutliner.h"
#include "LevelEditorViewport.h"
#include "LevelEditor.h"
#include "Editor.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Text/STextBlock.h"
#include "WorldPartitionEditorModule.h"

TMap<FName, SWorldPartitionEditorGrid::PartitionEditorGridCreateInstanceFunc> SWorldPartitionEditorGrid::PartitionEditorGridCreateInstanceFactory;

#define LOCTEXT_NAMESPACE "WorldPartitionEditor"

void SWorldPartitionEditorGrid::Construct(const FArguments& InArgs)
{
	World = InArgs._InWorld;
	WorldPartition = World.IsValid() ? World->GetWorldPartition() : nullptr;

	if (IsDisabled())
	{
		const FText Message = LOCTEXT("WorldPartitionIsDisabled", "World Partition is disabled for this map");

		ChildSlot
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(STextBlock)
			.Text(Message)
			.ColorAndOpacity(FSlateColor::UseForeground())
		];
	}
	SetClipping(EWidgetClipping::ClipToBounds);
}

void SWorldPartitionEditorGrid::RegisterPartitionEditorGridCreateInstanceFunc(FName Name, PartitionEditorGridCreateInstanceFunc CreateFunc)
{
	PartitionEditorGridCreateInstanceFactory.Add(Name, CreateFunc);
}

SWorldPartitionEditorGrid::PartitionEditorGridCreateInstanceFunc SWorldPartitionEditorGrid::GetPartitionEditorGridCreateInstanceFunc(FName Name)
{
	return *PartitionEditorGridCreateInstanceFactory.Find(Name);
}

bool SWorldPartitionEditorGrid::GetPlayerView(FVector& Location, FRotator& Rotation) const
{
	// We are in the PIE
	if (GEditor->PlayWorld)
	{
		for (FConstPlayerControllerIterator Iterator = GEditor->PlayWorld->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			if (APlayerController* PlayerActor = Iterator->Get())
			{
				PlayerActor->GetPlayerViewPoint(Location, Rotation);
				return true;
			}
		}
	}
	
	return false;
}

bool SWorldPartitionEditorGrid::GetObserverView(FVector& Location, FRotator& Rotation) const
{
	// We are in the SIE
	if (GEditor->bIsSimulatingInEditor && GCurrentLevelEditingViewportClient->IsSimulateInEditorViewport())
	{
		Rotation = GCurrentLevelEditingViewportClient->GetViewRotation();
		Location = GCurrentLevelEditingViewportClient->GetViewLocation();
		return true;
	}

	// We are in the editor world
	if (!GEditor->PlayWorld)
	{
		if (FWorldPartitionEditorModule::GetActiveLevelViewportCameraInfo(Location, Rotation))
		{
			return true;
		}
	}

	return false;
}

void SWorldPartitionEditorGrid::Refresh()
{
	TWeakPtr<class ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor")).GetLevelEditorInstance();
	if (LevelEditor.IsValid())
	{
		TArray<TWeakPtr<class ISceneOutliner>> SceneOutlinerPtrs = LevelEditor.Pin()->GetAllSceneOutliners();

		for (TWeakPtr<class ISceneOutliner> SceneOutlinerPtr : SceneOutlinerPtrs)
		{
			if (TSharedPtr<class ISceneOutliner> SceneOutlinerPin = SceneOutlinerPtr.Pin())
			{
				SceneOutlinerPin->FullRefresh();
			}
		}
	}
}

bool SWorldPartitionEditorGrid::IsDisabled() const
{
	return !WorldPartition.IsValid();
}

#undef LOCTEXT_NAMESPACE
