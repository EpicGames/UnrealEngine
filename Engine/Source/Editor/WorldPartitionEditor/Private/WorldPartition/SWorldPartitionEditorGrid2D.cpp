// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/SWorldPartitionEditorGrid2D.h"
#include "ActorFactories/ActorFactory.h"
#include "Brushes/SlateColorBrush.h"
#include "Builders/CubeBuilder.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Selection.h"
#include "Engine/Texture2D.h"
#include "EngineModule.h"
#include "EngineUtils.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Layout/WidgetPath.h"
#include "LevelEditorViewport.h"
#include "LevelEditorActions.h"
#include "LocationVolume.h"
#include "Modules/ModuleManager.h"
#include "Rendering/SlateRenderer.h"
#include "ScopedTransaction.h"
#include "SWorldPartitionViewportWidget.h"
#include "SEditorViewportToolBarMenu.h"
#include "Styling/StyleColors.h"
#include "TextureResource.h"
#include "ToolMenus.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "WorldBrowserModule.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODCreationFilter.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterShape.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"
#include "WorldPartition/WorldPartitionEditorHash.h"
#include "WorldPartition/WorldPartitionEditorLoaderAdapter.h"
#include "WorldPartition/WorldPartitionEditorPerProjectUserSettings.h"
#include "WorldPartition/WorldPartitionEditorSettings.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionHLODsBuilder.h"
#include "WorldPartition/WorldPartitionMiniMap.h"
#include "WorldPartition/WorldPartitionMiniMapHelper.h"
#include "WorldPartition/WorldPartitionMiniMapVolume.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/AxisDisplayInfo.h"

#include "IAssetViewport.h"
#include "Kismet2/DebuggerCommands.h"

#define LOCTEXT_NAMESPACE "WorldPartitionEditor"

static bool GShowEditorProfilingStats = 0;
static FAutoConsoleCommand CVarToggleShowEditorProfilingStats(
	TEXT("wp.Editor.ToggleShowEditorProfiling"),
	TEXT("Toggles showing editor profiling stats."),
	FConsoleCommandDelegate::CreateLambda([] { GShowEditorProfilingStats = !GShowEditorProfilingStats; })
);

class FWeightedMovingAverageScope
{
public:
	FORCEINLINE FWeightedMovingAverageScope(double& InValue)
		: Value(InValue)
	{
		Time = -FPlatformTime::Seconds();
	}

	FORCEINLINE ~FWeightedMovingAverageScope()
	{
		Value = FMath::WeightedMovingAverage(Time + FPlatformTime::Seconds(), Value, 0.05);
	}

private:
	double Time;
	double& Value;
};

class F2DRectBooleanSubtract
{
public:
	F2DRectBooleanSubtract(const FBox2D& Rect)
	{
		Rects.Add(Rect);
	};

	void SubRect(const FBox2D& SubRect)
	{
		TArray<FBox2D> NewRects;

		for (auto It(Rects.CreateIterator()); It; ++It)
		{
			if (SubRect.Intersect(*It))
			{
				if (SubRect.IsInsideOrOn(It->Min) && SubRect.IsInsideOrOn(It->Max))
				{
					It.RemoveCurrent();
					continue;
				}

				const FBox2D ClipRect = It->Overlap(SubRect);

				if (ClipRect.GetArea())
				{
					const bool bClipTop = (ClipRect.Min.Y > It->Min.Y) && (ClipRect.Min.Y < It->Max.Y);
					const bool bClipBottom = (ClipRect.Max.Y < It->Max.Y) && (ClipRect.Max.Y > It->Min.Y);
					const bool bClipLeft = (ClipRect.Min.X > It->Min.X) && (ClipRect.Min.X < It->Max.X);
					const bool bClipRight = (ClipRect.Max.X < It->Max.X) && (ClipRect.Max.X > It->Min.X);

					if (bClipTop || bClipBottom || bClipLeft || bClipRight)
					{
						if (bClipTop)
						{
							NewRects.Emplace(It->Min, FVector2D(It->Max.X, ClipRect.Min.Y));
						}

						if (bClipBottom)
						{
							NewRects.Emplace(FVector2D(It->Min.X, ClipRect.Max.Y), It->Max);
						}

						if (bClipLeft)
						{
							NewRects.Emplace(FVector2D(It->Min.X, ClipRect.Min.Y), FVector2D(ClipRect.Min.X, ClipRect.Max.Y));
						}

						if (bClipRight)
						{
 							NewRects.Emplace(FVector2D(ClipRect.Max.X, ClipRect.Min.Y), FVector2D(It->Max.X, ClipRect.Max.Y));
						}

						It.RemoveCurrent();
					}
				}
			}
		}

		Rects += NewRects;
	}

	const TArray<FBox2D> GetRects() const
	{
		return Rects;
	}

private:
	TArray<FBox2D> Rects;
};

static bool IsBoundsSelected(const FBox& SelectBox, const FBox& Bounds)
{
	return SelectBox.IsValid && Bounds.IntersectXY(SelectBox) && !Bounds.IsInsideXY(SelectBox);
}

static bool IsBoundsHovered(const FVector2D& Point, const FBox& Bounds)
{
	return Bounds.IsInsideOrOnXY(FVector(Point.X, Point.Y, 0));
}

static bool IsBoundsEdgeHovered(const FVector2D& Point, const FBox2D& Bounds, float Size)
{
	const float DistanceToPoint = FMath::Sqrt(Bounds.ExpandBy(-Size * 0.5f).ComputeSquaredDistanceToPoint(Point));
	return DistanceToPoint > UE_KINDA_SMALL_NUMBER && DistanceToPoint < Size;
}

static bool HitTestZFromLocation(UWorld* World, const FVector2D& WorldLocation, FHitResult& OutResult)
{
	const FVector TraceStart(WorldLocation.X, WorldLocation.Y, HALF_WORLD_MAX);
	const FVector TraceEnd(WorldLocation.X, WorldLocation.Y, -HALF_WORLD_MAX);
	const FCollisionQueryParams TraceParams(SCENE_QUERY_STAT(SWorldPartitionEditorGrid2D_HitTestZFromLocation), true);
	return World->LineTraceSingleByChannel(OutResult, TraceStart, TraceEnd, ECC_Visibility, TraceParams);
}

const TSet<TObjectPtr<UWorldPartitionEditorLoaderAdapter>>& GetRegisteredEditorLoaderAdapters(UWorldPartition* WorldPartition)
{
	static const TSet<TObjectPtr<UWorldPartitionEditorLoaderAdapter>> EmptyRegisteredEditorLoaderAdapters;
	return WorldPartition->IsStreamingEnabled() ? WorldPartition->GetRegisteredEditorLoaderAdapters() : EmptyRegisteredEditorLoaderAdapters;
}

template <class T>
void ForEachIntersectingLoaderAdapters(UWorldPartition* WorldPartition, const FBox& SelectBox, T Func)
{
	for (UWorldPartitionEditorLoaderAdapter* EditorLoaderAdapter : GetRegisteredEditorLoaderAdapters(WorldPartition))
	{
		if (IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = EditorLoaderAdapter->GetLoaderAdapter())
		{
			if (LoaderAdapter->GetBoundingBox().IsSet() && IsBoundsSelected(SelectBox, *LoaderAdapter->GetBoundingBox()))
			{
				Func(EditorLoaderAdapter);
			}
		}
	}

	FWorldPartitionHelpers::ForEachIntersectingActorDescInstance(WorldPartition, SelectBox, [&](const FWorldPartitionActorDescInstance* ActorDescInstance)
	{
		if (AActor* Actor = ActorDescInstance->GetActor())
		{
			if (ActorDescInstance->GetActorNativeClass()->ImplementsInterface(UWorldPartitionActorLoaderInterface::StaticClass()))
			{
				if (IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = Cast<IWorldPartitionActorLoaderInterface>(Actor)->GetLoaderAdapter())
				{
					if (IsBoundsSelected(SelectBox, ActorDescInstance->GetEditorBounds()))
					{
						Func(Actor);
					}
				}
			}
		}

		return true;
	});
}

SWorldPartitionEditorGrid2D::FEditorCommands::FEditorCommands()
	: TCommands<FEditorCommands>
(
	"WorldPartition",
	NSLOCTEXT("Contexts", "WorldPartition", "World Partition"),
	NAME_None,
	FAppStyle::GetAppStyleSetName()
)
{}

void SWorldPartitionEditorGrid2D::FEditorCommands::RegisterCommands()
{
	// Context Menu
	UI_COMMAND(CreateRegionFromSelection, "Load Region From Selection", "Load region from selection.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(BuildHLODsForSelection, "Build HLODs For Selection", "Build HLODs in selected area.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(BuildHLODsForSelectedRegions, "Build HLODs For Selected Region(s)", "Build HLODs for the selected region(s).", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ConvertSelectedRegionsToActors, "Convert Selected Region(s) To Actor(s)", "Convert the selected region(s) to actor(s).", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(LoadSelectedRegions, "Load Selected Region(s)", "Load the selected region(s).", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(UnloadSelectedRegions, "Unload Selected Region(s)", "Unload the selected region(s).", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(MoveCameraHere, "Move Camera Here", "Move the camera to the selected location.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PlayFromHere, "Play From Here", "Play from here.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(LoadFromHere, "Load From Here", "Load from here.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(BugItHere, "Bug It Here", "Log BugItGo command of the selected location to console (also adds it to clipboard).", EUserInterfaceActionType::Button, FInputChord());

	// Toolbar
	UI_COMMAND(FollowPlayerInPIE, "Follow Player In PIE", "Follow Player In PIE.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BugItGoLoadRegion, "BugItGo Load Region", "Using BugItGo command, it will create a loading region and zoom on it.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(ShowActors, "Actor(s)", "Show Actor(s).", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowHLODActors, "HLOD Actor(s)", "Show HLOD Actor(s).", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowGrid, "Grid", "Show Grid.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowMiniMap, "Minimap", "Show the minimap texture.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowCoords, "Coordinates", "Show grid cell coordinates, you might need to zoom closer to see them.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowMouseCoords, "Mouse Coordinates", "Show mouse world coordinates.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(ShowLoadingRegions, "Loading Regions", "Show loading regions.", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(FocusSelection, "Focus Selection", "Focus Selection.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(FocusLoadedRegions, "Focus Loaded Regions", "Focus Loaded Regions.", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(FocusWorld, "Focus World", "Focus World.", EUserInterfaceActionType::ToggleButton, FInputChord());
}

void SWorldPartitionEditorGrid2D::SToolBar::Construct(const FArguments& InArgs)
{
	WPEditorGrid2D = InArgs._WPEditorGrid2D;
	check(WPEditorGrid2D);
	CommandList = WPEditorGrid2D->CommandList;

	auto MakeQuickActionsToolBarWidget = [this]()
	{
		FSlimHorizontalToolBarBuilder ToolbarBuilder(CommandList, FMultiBoxCustomization::None);

		FName ToolBarStyle = "EditorViewportToolBar";
		ToolbarBuilder.SetStyle(&FAppStyle::Get(), ToolBarStyle);
		ToolbarBuilder.SetLabelVisibility(EVisibility::Collapsed);

		const FEditableTextBoxStyle& TextBoxStyle = FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");

		ToolbarBuilder.BeginSection("QuickActions");
		{
			ToolbarBuilder.BeginBlockGroup();

			const FEditorCommands& Commands = FEditorCommands::Get();
			ToolbarBuilder.AddToolBarButton(Commands.FocusSelection, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>());
			ToolbarBuilder.AddToolBarButton(Commands.FocusLoadedRegions, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>());
			ToolbarBuilder.AddToolBarButton(Commands.FocusWorld, NAME_None, TAttribute<FText>(), TAttribute<FText>(), TAttribute<FSlateIcon>());

			ToolbarBuilder.EndBlockGroup();
		}
		ToolbarBuilder.EndSection();

		return ToolbarBuilder.MakeWidget();
	};

	const FMargin ToolbarSlotPadding(4.0f, 1.0f);
	const FMargin ToolbarButtonPadding(4.0f, 0.0f);

	TSharedPtr<SHorizontalBox> MainBoxPtr;

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("EditorViewportToolBar.Background"))
		.Cursor(EMouseCursor::Default)
		[
			SAssignNew(MainBoxPtr, SHorizontalBox)
		]
	];

	// Options menu
	MainBoxPtr->AddSlot()
		.AutoWidth()
		.Padding(ToolbarSlotPadding)
		[
			SNew(SEditorViewportToolbarMenu)
			.ParentToolBar(SharedThis(this))
			.Cursor(EMouseCursor::Default)
			.Image("EditorViewportToolBar.OptionsDropdown")
			.OnGetMenuContent(this, &SToolBar::GenerateOptionsMenu)
			.Visibility(this, &SToolBar::IsOptionsMenuVisible)
		];

	// Show menu
	MainBoxPtr->AddSlot()
		.AutoWidth()
		.Padding(ToolbarSlotPadding)
		[
			SNew(SEditorViewportToolbarMenu)
			.Label(LOCTEXT("ShowMenuTitle", "Show"))
			.Cursor(EMouseCursor::Default)
			.ParentToolBar(SharedThis(this))
			.OnGetMenuContent(this, &SToolBar::GenerateShowMenu)
		];

	// Build menu
	MainBoxPtr->AddSlot()
		.AutoWidth()
		.Padding(ToolbarSlotPadding)
		[
			SNew(SEditorViewportToolbarMenu)
			.Label(LOCTEXT("BuildMenuTitle", "Build"))
			.Cursor(EMouseCursor::Default)
			.ParentToolBar(SharedThis(this))
			.OnGetMenuContent(this, &SToolBar::GenerateBuildMenu)
		];

	// Quick Actions Toolbar
	MainBoxPtr->AddSlot()
		.Padding(ToolbarSlotPadding)
		.HAlign(HAlign_Right)
		[
			MakeQuickActionsToolBarWidget()
		];
}

EVisibility SWorldPartitionEditorGrid2D::SToolBar::IsOptionsMenuVisible() const
{
	if (GetDefault<UWorldPartitionEditorSettings>()->GetDisablePIE() && !GetDefault<UWorldPartitionEditorSettings>()->GetEnableLoadingInEditor())
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

TSharedRef<SWidget> SWorldPartitionEditorGrid2D::SToolBar::GenerateUnloadedOpacitySlider() const
{	
	const float OpacityMinValue = 0.5f;
	const float OpacityMaxValue = 0.8f;

	return
		SNew(SSlider)
		.ToolTipText(LOCTEXT("MinimapUnloadedOpactiyToolTip", "Adjust the opacity of the unloaded regions of the minimap."))
		.MinValue(OpacityMinValue)
		.MaxValue(OpacityMaxValue)
		.Value(WPEditorGrid2D, &SWorldPartitionEditorGrid2D::GetMiniMapUnloadedOpacity)
		.OnValueChanged(WPEditorGrid2D, &SWorldPartitionEditorGrid2D::SetMiniMapUnloadedOpacity)
		.OnMouseCaptureEnd(WPEditorGrid2D, &SWorldPartitionEditorGrid2D::SaveMiniMapUnloadedOpacityUserSetting)
		.IsEnabled(WPEditorGrid2D, &SWorldPartitionEditorGrid2D::IsMiniMapUnloadedOpacityEnabled);
}

TSharedRef<SWidget> SWorldPartitionEditorGrid2D::SToolBar::GenerateOptionsMenu() const
{
	static const FName MenuName(TEXT("WorldPartition.OptionsMenu"));

	const FEditorCommands& Commands = FEditorCommands::Get();
	UToolMenu* OptionsMenu = UToolMenus::Get()->RegisterMenu(MenuName);

	static const FName SectionGeneralName(TEXT("Options.General"));
	FToolMenuSection& SectionGeneral = OptionsMenu->FindOrAddSection(SectionGeneralName);
	SectionGeneral.AddMenuEntry(Commands.FollowPlayerInPIE);
	SectionGeneral.AddMenuEntry(Commands.BugItGoLoadRegion);

	static const FName SectionMinimapName(TEXT("Options.Minimap"));
	FToolMenuSection& SectionMinimap = OptionsMenu->AddSection(SectionMinimapName, LOCTEXT("WorldPartitionOptionsMenuMinimap", "Minimap"));
	static const FName SliderUnloadedOpacityName(TEXT("UnloadedOpacity"));
	SectionMinimap.AddEntry(FToolMenuEntry::InitWidget(SliderUnloadedOpacityName, GenerateUnloadedOpacitySlider(), LOCTEXT("UnloadedOpacity", "Unloaded Opacity"), true));	

	return UToolMenus::Get()->GenerateWidget(MenuName, FToolMenuContext(CommandList));
}

TSharedRef<SWidget> SWorldPartitionEditorGrid2D::SToolBar::GenerateShowMenu() const
{
	static const FName MenuName(TEXT("WorldPartition.ShowMenu"));
	static const FName SectionName(TEXT("Show"));

	const FEditorCommands& Commands = FEditorCommands::Get();
	UToolMenu* ShowMenu = UToolMenus::Get()->RegisterMenu(MenuName);

	FToolMenuSection& Section = ShowMenu->FindOrAddSection(SectionName);
	Section.AddMenuEntry(Commands.ShowActors);
	Section.AddMenuEntry(Commands.ShowHLODActors);
	Section.AddMenuEntry(Commands.ShowGrid);
	Section.AddMenuEntry(Commands.ShowMiniMap);
	Section.AddMenuEntry(Commands.ShowCoords);
	Section.AddMenuEntry(Commands.ShowMouseCoords);
	Section.AddMenuEntry(Commands.ShowLoadingRegions);

	return UToolMenus::Get()->GenerateWidget(MenuName, FToolMenuContext(CommandList));
}

TSharedRef<SWidget> SWorldPartitionEditorGrid2D::SToolBar::GenerateBuildMenu() const
{
	static const FName MenuName(TEXT("WorldPartition.BuildMenu"));
	static const FName SectionName(TEXT("Build"));

	UToolMenu* BuildMenu = UToolMenus::Get()->RegisterMenu(MenuName);

	FToolMenuSection& Section = BuildMenu->FindOrAddSection(SectionName);
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	const FLevelEditorCommands& LevelEditorCommands = LevelEditorModule.GetLevelEditorCommands();
	Section.AddMenuEntry(LevelEditorCommands.BuildHLODs);
	Section.AddMenuEntry(LevelEditorCommands.BuildHLODs_SelectedActors);
	Section.AddMenuEntry(LevelEditorCommands.BuildMinimap);
	Section.AddMenuEntry(LevelEditorCommands.BuildLandscapeSplineMeshes);

	return UToolMenus::Get()->GenerateWidget(MenuName, FToolMenuContext(LevelEditorModule.GetGlobalLevelEditorActions()));
}

SWorldPartitionEditorGrid2D::SWorldPartitionEditorGrid2D()
	: CommandList(MakeShareable(new FUICommandList))
	, ChildSlot(this)
	, Scale(0.001f)
	, Trans(ForceInit)
	, ScreenRect(ForceInit)
	, bIsDragSelecting(false)
	, bIsPanning(false)
	, bIsMeasuring(false)
	, bShowActors(false)
	, bShowHLODActors(false)
	, bShowGrid(true)
	, bShowLoadingRegions(true)
	, bShowMiniMap(true)
	, bShowMouseCoords(false)
	, bFollowPlayerInPIE(false)
	, SelectBox(ForceInit)
	, SelectBoxGridSnapped(ForceInit)
	, WorldMiniMapBounds(ForceInit)
	, TickTime(0)
	, PaintTime(0)
{
	FEditorCommands::Register();
	
	FWorldBrowserModule& WorldBrowserModule = FModuleManager::LoadModuleChecked<FWorldBrowserModule>("WorldBrowser");
	WorldBrowserModule.OnShutdown().AddLambda([](){ FEditorCommands::Unregister(); });

	MiniMapUnloadedOpacity = GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>()->GetMinimapUnloadedOpacity();
}

SWorldPartitionEditorGrid2D::~SWorldPartitionEditorGrid2D()
{}

void SWorldPartitionEditorGrid2D::Construct(const FArguments& InArgs)
{
	SWorldPartitionEditorGrid::Construct(SWorldPartitionEditorGrid::FArguments().InWorld(InArgs._InWorld));

	// Defaults
	Trans = FVector2D(0, 0);
	Scale = 0.00133333332f;
	TotalMouseDelta = 0;
	SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);

	if (GetWorldPartition())
	{
		UpdateWorldMiniMapDetails();
		bShowActors = !WorldMiniMapBrush.HasUObject();
	}

	// UI
	const FMargin ToolbarSlotPadding(4.0f, 1.0f);
	const float BottomPartMinDesiredHeight(25.0f); // Need this because the left part is a widget bigger than other text parts and his visibility could be turn off resulting in moving the text on screen (unwanted behavior).

	ChildSlot
	[
		SNew(SOverlay)

		// Toolbar
		+ SOverlay::Slot()
		.VAlign(VAlign_Top)
		.Padding(ToolbarSlotPadding)
		[
			SNew(SToolBar)
			.WPEditorGrid2D(this)
		]

		// Bottom section
		+SOverlay::Slot()
		.VAlign(VAlign_Bottom)

		.Padding(10.f, 0.f, 0.f, 1.f)
		[
			SNew(SUniformGridPanel)
			.SlotPadding(2)
			 + SUniformGridPanel::Slot(0, 0)
			[
				SNew(SBox)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SAssignNew(ViewportWidget, SWorldPartitionViewportWidget)
					.Clickable(false)
					.Visibility_Lambda([this]() { return ViewportWidget->GetVisibility(GetWorld()); })
				]
			]
			+ SUniformGridPanel::Slot(1, 0)
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.MinDesiredHeight(BottomPartMinDesiredHeight)
				[
					SAssignNew(TextWorldBoundsInKMWidget, STextBlock)
				]
			]
			+ SUniformGridPanel::Slot(2, 0)
			[
				SNew(SBox)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.MinDesiredHeight(BottomPartMinDesiredHeight)
				[
					SAssignNew(TextRulerWidget, STextBlock)
				]
			]
		]
	];

	BindCommands();

	ExternalDirtyActorsTracker.Reset();
	if (InArgs._InWorld)
	{
		ExternalDirtyActorsTracker = MakeUnique<FExternalDirtyActorsTracker>(InArgs._InWorld->PersistentLevel, this);
	}
}

void SWorldPartitionEditorGrid2D::BindCommands()
{
	auto CanCreateRegionFromSelection = [this]()
	{
		return !!SelectBoxGridSnapped.IsValid;
	};

	auto CanLoadUnloadSelectedRegions = [this](bool bLoad)
	{
		for (const TWeakInterfacePtr<IWorldPartitionActorLoaderInterface>& SelectedLoaderAdapter : SelectedLoaderInterfaces)
		{
			if (IWorldPartitionActorLoaderInterface* LoaderInterface = SelectedLoaderAdapter.Get())
			{
				if (bLoad != LoaderInterface->GetLoaderAdapter()->IsLoaded())
				{
					return true;
				}
			}
		}

		return false;
	};

	auto CanBuildHLODsForSelectedRegions = [this]()
	{
		for (const TWeakInterfacePtr<IWorldPartitionActorLoaderInterface>& SelectedLoaderAdapter : SelectedLoaderInterfaces)
		{
			if (IWorldPartitionActorLoaderInterface* LoaderInterface = SelectedLoaderAdapter.Get())
			{
				if (LoaderInterface->GetLoaderAdapter()->GetBoundingBox().IsSet())
				{
					return true;
				}
			}
		}
		return false;
	};

	auto CanConvertSelectedRegionsToActors = [this]()
	{
		for (const TWeakInterfacePtr<IWorldPartitionActorLoaderInterface>& SelectedLoaderAdapter : SelectedLoaderInterfaces)
		{
			if (UWorldPartitionEditorLoaderAdapter* EditorLoaderAdapter = Cast<UWorldPartitionEditorLoaderAdapter>(SelectedLoaderAdapter.Get()))
			{
				return true;
			}
		};
		return false;
	};

	auto CanLoadSelectedRegions = [this, CanLoadUnloadSelectedRegions]() { return CanLoadUnloadSelectedRegions(true); };
	auto CanUnloadSelectedRegions = [this, CanLoadUnloadSelectedRegions]() { return CanLoadUnloadSelectedRegions(false); };

	auto CanLoadFromHere = [this]()
	{
		return GetWorldPartition()->IsStreamingEnabled();
	};

	auto CanFocusSelection = [this]()
	{
		if (UWorldPartitionSubsystem* WorldPartitionSubsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(GetWorld()))
		{
			return IsInteractive() && (GEditor->GetSelectedActors()->Num() || WorldPartitionSubsystem->SelectedActorHandles.Num());
		}

		return false;
	};


	const FEditorCommands& Commands = FEditorCommands::Get();

	CommandList->MapAction(Commands.CreateRegionFromSelection, FExecuteAction::CreateSP(this, &SWorldPartitionEditorGrid2D::CreateRegionFromSelection), FCanExecuteAction::CreateLambda(CanCreateRegionFromSelection));
	CommandList->MapAction(Commands.BuildHLODsForSelection, FExecuteAction::CreateSP(this, &SWorldPartitionEditorGrid2D::BuildHLODsForSelection), FCanExecuteAction::CreateLambda(CanCreateRegionFromSelection));
	CommandList->MapAction(Commands.BuildHLODsForSelectedRegions, FExecuteAction::CreateSP(this, &SWorldPartitionEditorGrid2D::BuildHLODsForSelectedRegions), FCanExecuteAction::CreateLambda(CanBuildHLODsForSelectedRegions));
	CommandList->MapAction(Commands.LoadSelectedRegions, FExecuteAction::CreateSP(this, &SWorldPartitionEditorGrid2D::LoadSelectedRegions), FCanExecuteAction::CreateLambda(CanLoadSelectedRegions));
	CommandList->MapAction(Commands.UnloadSelectedRegions, FExecuteAction::CreateSP(this, &SWorldPartitionEditorGrid2D::UnloadSelectedRegions), FCanExecuteAction::CreateLambda(CanUnloadSelectedRegions));
	CommandList->MapAction(Commands.ConvertSelectedRegionsToActors, FExecuteAction::CreateSP(this, &SWorldPartitionEditorGrid2D::ConvertSelectedRegionsToActors), FCanExecuteAction::CreateLambda(CanConvertSelectedRegionsToActors));
	CommandList->MapAction(Commands.MoveCameraHere, FExecuteAction::CreateSP(this, &SWorldPartitionEditorGrid2D::MoveCameraHere));
	CommandList->MapAction(Commands.PlayFromHere, FExecuteAction::CreateSP(this, &SWorldPartitionEditorGrid2D::PlayFromHere));
	CommandList->MapAction(Commands.LoadFromHere, FExecuteAction::CreateSP(this, &SWorldPartitionEditorGrid2D::LoadFromHere), FCanExecuteAction::CreateLambda(CanLoadFromHere));
	CommandList->MapAction(Commands.BugItHere, FExecuteAction::CreateSP(this, &SWorldPartitionEditorGrid2D::BugItHere));

	// Options
	CommandList->MapAction(Commands.FollowPlayerInPIE, FExecuteAction::CreateLambda([this]() { bFollowPlayerInPIE = !bFollowPlayerInPIE; }), FCanExecuteAction(), FIsActionChecked::CreateLambda([this]() { return bFollowPlayerInPIE; }), FIsActionButtonVisible::CreateLambda([this]() { return !GetDefault<UWorldPartitionEditorSettings>()->GetDisablePIE(); }));
	CommandList->MapAction(Commands.BugItGoLoadRegion, FExecuteAction::CreateLambda([this]() { GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>()->SetBugItGoLoadRegion(!GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>()->GetBugItGoLoadRegion()); }), FCanExecuteAction(), FIsActionChecked::CreateLambda([this]() { return GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>()->GetBugItGoLoadRegion(); }), FIsActionButtonVisible::CreateLambda([this]() { return GetDefault<UWorldPartitionEditorSettings>()->GetEnableLoadingInEditor(); }));

	// Show toggles
	CommandList->MapAction(Commands.ShowActors, FExecuteAction::CreateLambda([this]() { bShowActors = !bShowActors; InvalidateShownActorsCache(); }), FCanExecuteAction(), FIsActionChecked::CreateLambda([this]() { return bShowActors; }));
	CommandList->MapAction(Commands.ShowHLODActors, FExecuteAction::CreateLambda([this]() { bShowHLODActors = !bShowHLODActors; InvalidateShownActorsCache(); }), FCanExecuteAction(), FIsActionChecked::CreateLambda([this]() { return bShowHLODActors; }));
	CommandList->MapAction(Commands.ShowGrid, FExecuteAction::CreateLambda([this]() { bShowGrid = !bShowGrid; }), FCanExecuteAction(), FIsActionChecked::CreateLambda([this]() { return bShowGrid; }));
	CommandList->MapAction(Commands.ShowMiniMap, FExecuteAction::CreateLambda([this]() { bShowMiniMap = !bShowMiniMap; }), FCanExecuteAction(), FIsActionChecked::CreateLambda([this]() { return bShowMiniMap; }));
	CommandList->MapAction(Commands.ShowCoords, FExecuteAction::CreateLambda([this]() { GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>()->SetShowCellCoords(!GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>()->GetShowCellCoords()); }), FCanExecuteAction(), FIsActionChecked::CreateLambda([this]() { return GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>()->GetShowCellCoords(); }), FIsActionButtonVisible::CreateLambda([this]() { return (GetWorldPartition() && GetWorldPartition()->IsStreamingEnabled()); }));
	CommandList->MapAction(Commands.ShowMouseCoords, FExecuteAction::CreateLambda([this]() { bShowMouseCoords = !bShowMouseCoords; }), FCanExecuteAction(), FIsActionChecked::CreateLambda([this]() { return bShowMouseCoords; }));
	CommandList->MapAction(Commands.ShowLoadingRegions, FExecuteAction::CreateLambda([this]() { bShowLoadingRegions = !bShowLoadingRegions; }), FCanExecuteAction(), FIsActionChecked::CreateLambda([this]() { return bShowLoadingRegions; }));

	// Quick Actions
	CommandList->MapAction(Commands.FocusSelection, FExecuteAction::CreateSP(this, &SWorldPartitionEditorGrid2D::FocusSelection), FCanExecuteAction::CreateLambda(CanFocusSelection));
	CommandList->MapAction(Commands.FocusLoadedRegions, FExecuteAction::CreateSP(this, &SWorldPartitionEditorGrid2D::FocusLoadedRegions), FCanExecuteAction::CreateLambda([this]() { return IsInteractive() && GetWorldPartition() && GetWorldPartition()->HasLoadedUserCreatedRegions(); }), FIsActionChecked(), FIsActionButtonVisible::CreateLambda([this]() { return GetDefault<UWorldPartitionEditorSettings>()->GetEnableLoadingInEditor(); }));
	CommandList->MapAction(Commands.FocusWorld, FExecuteAction::CreateSP(this, &SWorldPartitionEditorGrid2D::FocusWorld), FCanExecuteAction::CreateLambda([this]() { return IsInteractive(); }));
}

void SWorldPartitionEditorGrid2D::SaveMiniMapUnloadedOpacityUserSetting()
{
	GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>()->SetMinimapUnloadedOpacity(MiniMapUnloadedOpacity);
}

bool SWorldPartitionEditorGrid2D::IsMiniMapUnloadedOpacityEnabled() const
{
	if (!bShowMiniMap)
	{
		return false;
	}

	if (!GetDefault<UWorldPartitionEditorSettings>()->GetEnableLoadingInEditor())
	{
		return false;
	}

	if (!GetWorldPartition()->IsStreamingEnabled())
	{
		return false;
	}

	UTexture2D* Texture2D = Cast<UTexture2D>(WorldMiniMapBrush.GetResourceObject());
	if (Texture2D == nullptr)
	{
		return false;
	}

	return true;
}

void SWorldPartitionEditorGrid2D::UpdateWorldMiniMapDetails()
{
	if (AWorldPartitionMiniMap* WorldMiniMap = FWorldPartitionMiniMapHelper::GetWorldPartitionMiniMap(GetWorld()))
	{
		WorldMiniMapBounds = FBox2D(FVector2D(WorldMiniMap->MiniMapWorldBounds.Min), FVector2D(WorldMiniMap->MiniMapWorldBounds.Max));
		if (UTexture2D* MiniMapTexture = WorldMiniMap->MiniMapTexture)
		{
			WorldMiniMapBrush.SetUVRegion(WorldMiniMap->UVOffset);
			WorldMiniMapBrush.SetImageSize(MiniMapTexture->GetImportedSize());
			WorldMiniMapBrush.SetResourceObject(MiniMapTexture);
		}
	}
}

void SWorldPartitionEditorGrid2D::CreateRegionFromSelection()
{
	const FBox RegionBox(FVector(SelectBoxGridSnapped.Min.X, SelectBoxGridSnapped.Min.Y, -HALF_WORLD_MAX), FVector(SelectBoxGridSnapped.Max.X, SelectBoxGridSnapped.Max.Y, HALF_WORLD_MAX));
	UWorldPartitionEditorLoaderAdapter* EditorLoaderAdapter = GetWorldPartition()->CreateEditorLoaderAdapter<FLoaderAdapterShape>(GetWorld(), RegionBox, TEXT("Loaded Region"));
	EditorLoaderAdapter->GetLoaderAdapter()->SetUserCreated(true);
	EditorLoaderAdapter->GetLoaderAdapter()->Load();
	
	ClearSelection();
	SelectedLoaderInterfaces.Add(EditorLoaderAdapter);

	GEditor->RedrawLevelEditingViewports();
	Refresh();
}

void SWorldPartitionEditorGrid2D::BuildHLODsForSelection()
{
	TArray<FBox> FilterVolumes = { FBox(FVector(SelectBoxGridSnapped.Min.X, SelectBoxGridSnapped.Min.Y, -HALF_WORLD_MAX), FVector(SelectBoxGridSnapped.Max.X, SelectBoxGridSnapped.Max.Y, HALF_WORLD_MAX)) };

	UWorldPartitionHLODsBuilder::BuildWithFilters(GetWorld(), { MakeShared<FHLODCreationVolumeFilter>(MoveTemp(FilterVolumes)) });

	ClearSelection();
	Refresh();
}

void SWorldPartitionEditorGrid2D::BuildHLODsForSelectedRegions()
{
	TArray<FBox> FilterVolumes;
	for (const TWeakInterfacePtr<IWorldPartitionActorLoaderInterface>& SelectedLoaderAdapter : SelectedLoaderInterfaces)
	{
		if (IWorldPartitionActorLoaderInterface* LoaderInterface = SelectedLoaderAdapter.Get())
		{
			TOptional<FBox> BoundingBox = LoaderInterface->GetLoaderAdapter()->GetBoundingBox();
			if (BoundingBox.IsSet())
			{
				FilterVolumes.Add(BoundingBox.GetValue());
			}
		}
	}

	UWorldPartitionHLODsBuilder::BuildWithFilters(GetWorld(), { MakeShared<FHLODCreationVolumeFilter>(MoveTemp(FilterVolumes)) });

	ClearSelection();
	Refresh();
}

void SWorldPartitionEditorGrid2D::LoadSelectedRegions()
{
	for (const TWeakInterfacePtr<IWorldPartitionActorLoaderInterface>& SelectedLoaderAdapter : SelectedLoaderInterfaces)
	{
		if (IWorldPartitionActorLoaderInterface* LoaderInterface = SelectedLoaderAdapter.Get())
		{
			LoaderInterface->GetLoaderAdapter()->Load();
		}
	}

	SelectBox.Init();
	SelectBoxGridSnapped.Init();

	GEditor->RedrawLevelEditingViewports();
	Refresh();
}

void SWorldPartitionEditorGrid2D::UnloadSelectedRegions()
{
	FLoaderInterfaceSet CopySelectedLoaderInterfaces(SelectedLoaderInterfaces);
	for (const TWeakInterfacePtr<IWorldPartitionActorLoaderInterface>& SelectedLoaderAdapter : CopySelectedLoaderInterfaces)
	{
		if (IWorldPartitionActorLoaderInterface* LoaderInterface = SelectedLoaderAdapter.Get())
		{
			LoaderInterface->GetLoaderAdapter()->Unload();

			if (UWorldPartitionEditorLoaderAdapter* EditorLoaderAdapter = Cast<UWorldPartitionEditorLoaderAdapter>(LoaderInterface))
			{
				SelectedLoaderInterfaces.Remove(EditorLoaderAdapter);
				GetWorldPartition()->ReleaseEditorLoaderAdapter(EditorLoaderAdapter);
			}
		}
	};
		
	SelectBox.Init();
	SelectBoxGridSnapped.Init();

	GEditor->RedrawLevelEditingViewports();
	Refresh();
}

void SWorldPartitionEditorGrid2D::ConvertSelectedRegionsToActors()
{
	FLoaderInterfaceSet TmpSelectedLoaderInterfaces = MoveTemp(SelectedLoaderInterfaces);
	ClearSelection();

	const FScopedTransaction Transaction(LOCTEXT("ConvertSelectedRegionsToActors", "Convert Selected Region(s) to Actor(s)"));
	const FBox WorldBounds = GetWorldPartition()->GetRuntimeWorldBounds();
	for (const TWeakInterfacePtr<IWorldPartitionActorLoaderInterface>& SelectedLoaderAdapter : TmpSelectedLoaderInterfaces)
	{
		if (UWorldPartitionEditorLoaderAdapter* EditorLoaderAdapter = Cast<UWorldPartitionEditorLoaderAdapter>(SelectedLoaderAdapter.Get()))
		{
			IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = EditorLoaderAdapter->GetLoaderAdapter();
			
			const FBox LoaderVolumeBox(*LoaderAdapter->GetBoundingBox());
			const FBox ActorVolumeBox(FVector(LoaderVolumeBox.Min.X, LoaderVolumeBox.Min.Y, WorldBounds.Min.Z), FVector(LoaderVolumeBox.Max.X, LoaderVolumeBox.Max.Y, WorldBounds.Max.Z));

			ALocationVolume* LocationVolume = GetWorld()->SpawnActor<ALocationVolume>(ActorVolumeBox.GetCenter(), FRotator::ZeroRotator);
			LocationVolume->Modify(true);
			FActorLabelUtilities::SetActorLabelUnique(LocationVolume, LocationVolume->GetActorLabel());
			
			UCubeBuilder* Builder = NewObject<UCubeBuilder>();
			Builder->Modify();
			FVector Extent = ActorVolumeBox.GetExtent();
			Builder->X = Extent.X * 2;
			Builder->Y = Extent.Y * 2;
			Builder->Z = FMath::Max(Extent.Z * 2, 200.0f);

			UActorFactory::CreateBrushForVolumeActor(LocationVolume, Builder);
			
			LocationVolume->GetLoaderAdapter()->Load();
			LoaderAdapter->Unload();
			
			GetWorldPartition()->ReleaseEditorLoaderAdapter(EditorLoaderAdapter);
		}
		else
		{
			SelectedLoaderInterfaces.Add(SelectedLoaderAdapter);
		}
	}

	GEditor->RedrawLevelEditingViewports();
	Refresh();
}

void SWorldPartitionEditorGrid2D::MoveCameraHere()
{
	FVector WorldLocation = FVector(MouseCursorPosWorld, 0);

	FHitResult HitResult;
	const bool bHitResultValid = HitTestZFromLocation(GetWorld(), MouseCursorPosWorld, HitResult);

	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		WorldLocation.Z = bHitResultValid ? HitResult.ImpactPoint.Z + 1000.0f : LevelVC->GetViewLocation().Z;

		LevelVC->SetViewLocation(WorldLocation);
		LevelVC->Invalidate();

		FEditorDelegates::OnEditorCameraMoved.Broadcast(WorldLocation, LevelVC->GetViewRotation(), LevelVC->ViewportType, LevelVC->ViewIndex);
	}
}

void SWorldPartitionEditorGrid2D::PlayFromHere()
{
	FVector StartLocation = FVector(MouseCursorPosWorld, 0);
	FRotator StartRotation;

	FHitResult HitResult;
	const bool bHitResultValid = HitTestZFromLocation(GetWorld(), MouseCursorPosWorld, HitResult);

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<IAssetViewport> ActiveLevelViewport = LevelEditorModule.GetFirstActiveViewport();

	if (ActiveLevelViewport.IsValid() && ActiveLevelViewport->GetAssetViewportClient().IsPerspective())
	{
		StartLocation.Z = bHitResultValid ? HitResult.ImpactPoint.Z : ActiveLevelViewport->GetAssetViewportClient().GetViewLocation().Z;
		StartRotation = ActiveLevelViewport->GetAssetViewportClient().GetViewRotation();

		FPlayWorldCommandCallbacks::StartPlayFromHere(StartLocation, StartRotation, ActiveLevelViewport);
	}	
}

void SWorldPartitionEditorGrid2D::LoadFromHere()
{
	const double SelectionSnap = GetSelectionSnap();
	const FVector LoadLocation = FVector(MouseCursorPosWorld, 0);
	const FVector LoadExtent(SelectionSnap, SelectionSnap, HALF_WORLD_MAX);
	FBox LoadCellsBox(LoadLocation - LoadExtent, LoadLocation + LoadExtent);

	// Snap box
	LoadCellsBox.Min.X = FMath::GridSnap(LoadCellsBox.Min.X - SelectionSnap / 2, SelectionSnap);
	LoadCellsBox.Min.Y = FMath::GridSnap(LoadCellsBox.Min.Y - SelectionSnap / 2, SelectionSnap);
	LoadCellsBox.Max.X = FMath::GridSnap(LoadCellsBox.Max.X + SelectionSnap / 2, SelectionSnap);
	LoadCellsBox.Max.Y = FMath::GridSnap(LoadCellsBox.Max.Y + SelectionSnap / 2, SelectionSnap);

	// Clip to minimap
	if (WorldMiniMapBounds.bIsValid && WorldMiniMapBounds.IsInside(FVector2D(MouseCursorPosWorld)))
	{
		LoadCellsBox.Min.X = FMath::Max(LoadCellsBox.Min.X, WorldMiniMapBounds.Min.X);
		LoadCellsBox.Min.Y = FMath::Max(LoadCellsBox.Min.Y, WorldMiniMapBounds.Min.Y);
		LoadCellsBox.Max.X = FMath::Min(LoadCellsBox.Max.X, WorldMiniMapBounds.Max.X);
		LoadCellsBox.Max.Y = FMath::Min(LoadCellsBox.Max.Y, WorldMiniMapBounds.Max.Y);
	}

	// Load box
	UWorldPartitionEditorLoaderAdapter* EditorLoaderAdapter = GetWorldPartition()->CreateEditorLoaderAdapter<FLoaderAdapterShape>(GetWorld(), LoadCellsBox, TEXT("Loaded Region"));
	EditorLoaderAdapter->GetLoaderAdapter()->SetUserCreated(true);
	EditorLoaderAdapter->GetLoaderAdapter()->Load();
}

void SWorldPartitionEditorGrid2D::BugItHere()
{
	FVector WorldLocation = FVector(MouseCursorPosWorld, 0);

	FHitResult HitResult;
	const bool bHitResultValid = HitTestZFromLocation(GetWorld(), MouseCursorPosWorld, HitResult);

	if (bHitResultValid && GCurrentLevelEditingViewportClient)
	{
		const FRotator ViewRotation = GCurrentLevelEditingViewportClient->GetViewRotation();
		const FString GoString = FString::Printf(TEXT("BugItGo %f %f %f %f %f %f"), MouseCursorPosWorld.X, MouseCursorPosWorld.Y, HitResult.Location.Z + 1000.0f, ViewRotation.Pitch, ViewRotation.Yaw, ViewRotation.Roll);
		
		UE_LOGF(LogWorldPartition, Log, "%ls", *GoString);

		FPlatformApplicationMisc::ClipboardCopy(*GoString);
	}
}

bool SWorldPartitionEditorGrid2D::IsFollowPlayerInPIE() const
{
	return bFollowPlayerInPIE && (UWorldPartition::IsSimulating() || GEditor->PlayWorld);
}

bool SWorldPartitionEditorGrid2D::IsInteractive() const
{
	return !IsFollowPlayerInPIE();
}

int64 SWorldPartitionEditorGrid2D::GetSelectionSnap() const
{
	return 1;
}

FReply SWorldPartitionEditorGrid2D::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const bool bIsLeftMouseButtonEffecting = MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton;
	const bool bIsRightMouseButtonEffecting = MouseEvent.GetEffectingButton() == EKeys::RightMouseButton;
	const bool bIsMiddleMouseButtonEffecting = MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton;
	const bool bIsRightMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton);
	const bool bIsLeftMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
	const bool bIsMiddleMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton);

	TotalMouseDelta = 0;

	if (bIsLeftMouseButtonEffecting || bIsMiddleMouseButtonEffecting || bIsRightMouseButtonEffecting)
	{
		FReply ReplyState = FReply::Handled();
		ReplyState.CaptureMouse(SharedThis(this));

		if (bIsLeftMouseButtonEffecting)
		{
			if (!MouseEvent.IsControlDown())
			{
				ClearSelection();
			}

			SelectionStart = MouseCursorPosWorld;
			SelectionEnd = SelectionStart;
			SelectBox.Init();
			SelectBoxGridSnapped.Init();
		}

		if (bIsMiddleMouseButtonEffecting)
		{
			MeasureStart = MeasureEnd = MouseCursorPosWorld;
		}

		return ReplyState;
	}

	return FReply::Unhandled();
}

TSharedRef<SWidget> SWorldPartitionEditorGrid2D::GenerateContextualMenu() const
{
	static const FName MenuName(TEXT("WorldPartition.ContextualMenu"));
	
	const FEditorCommands& Commands = FEditorCommands::Get();
	UToolMenu* ContextualMenu = UToolMenus::Get()->RegisterMenu(MenuName);

	if (GetDefault<UWorldPartitionEditorSettings>()->GetEnableLoadingInEditor())
	{
		static const FName SectionSelectionName(TEXT("ContextMenu.Selection"));
		FToolMenuSection& SectionSelection = ContextualMenu->AddSection(SectionSelectionName, LOCTEXT("WorldPartitionSelectionHeader", "Selection"));
		SectionSelection.AddMenuEntry(Commands.CreateRegionFromSelection);
		
		static const FName SectionRegionsName(TEXT("ContextMenu.Regions"));
		FToolMenuSection& SectionRegions = ContextualMenu->AddSection(SectionRegionsName, LOCTEXT("WorldPartitionRegionsHeader", "Region(s)"));
		SectionRegions.AddMenuEntry(Commands.LoadSelectedRegions);
		SectionRegions.AddMenuEntry(Commands.UnloadSelectedRegions);
		SectionRegions.AddSeparator(TEXT("RegionsSeparator"));
		SectionRegions.AddMenuEntry(Commands.ConvertSelectedRegionsToActors);
	}

	if (GetWorldPartition() && GetWorldPartition()->IsStreamingEnabled())
	{
		static const FName SectionHLODName(TEXT("ContextMenu.HLOD"));
		FToolMenuSection& SectionHLOD = ContextualMenu->AddSection(SectionHLODName, LOCTEXT("WorldPartitionHLODHeader", "HLOD"));
		SectionHLOD.AddMenuEntry(Commands.BuildHLODsForSelection);
		if (GetDefault<UWorldPartitionEditorSettings>()->GetEnableLoadingInEditor())
		{
			SectionHLOD.AddMenuEntry(Commands.BuildHLODsForSelectedRegions);
		}
	}

	static const FName SectionMiscName(TEXT("ContextMenu.Misc"));
	FToolMenuSection& SectionMisc = ContextualMenu->AddSection(SectionMiscName, LOCTEXT("WorldPartitionMiscHeader", "Misc"));
	SectionMisc.AddMenuEntry(Commands.MoveCameraHere);

	if (!GetDefault<UWorldPartitionEditorSettings>()->GetDisableBugIt())
	{
		SectionMisc.AddMenuEntry(Commands.BugItHere);
	}

	if (!GetDefault<UWorldPartitionEditorSettings>()->GetDisablePIE())
	{
		SectionMisc.AddMenuEntry(Commands.PlayFromHere);
	}

	if (GetDefault<UWorldPartitionEditorSettings>()->GetEnableLoadingInEditor())
	{
		SectionMisc.AddMenuEntry(Commands.LoadFromHere);
	}

	return UToolMenus::Get()->GenerateWidget(MenuName, FToolMenuContext(CommandList));
}

FReply SWorldPartitionEditorGrid2D::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const bool bIsLeftMouseButtonEffecting = MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton;
	const bool bIsRightMouseButtonEffecting = MouseEvent.GetEffectingButton() == EKeys::RightMouseButton;
	const bool bIsMiddleMouseButtonEffecting = MouseEvent.GetEffectingButton() == EKeys::MiddleMouseButton;
	const bool bIsRightMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton);
	const bool bIsLeftMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
	const bool bIsMiddleMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton);

	TotalMouseDelta = 0;

	if (bIsLeftMouseButtonEffecting || bIsMiddleMouseButtonEffecting || bIsRightMouseButtonEffecting)
	{
		FReply ReplyState = FReply::Handled();

		const bool bHasMouseCapture = bIsDragSelecting || bIsPanning || bIsMeasuring;
		MouseCursorPos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		MouseCursorPosWorld = ScreenToWorld.TransformPoint(MouseCursorPos);

		if (!bHasMouseCapture && bIsRightMouseButtonEffecting)
		{
			if (HoveredLoaderInterface.IsValid() && !SelectedLoaderInterfaces.Contains(HoveredLoaderInterface))
			{
				SelectedLoaderInterfaces.Reset();
				SelectedLoaderInterfaces.Add(HoveredLoaderInterface);
			}

			FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
			FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, GenerateContextualMenu(), MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
		}

		if (bIsLeftMouseButtonEffecting)
		{
			FLoaderInterfaceSet LoaderAdaptersToSelect;
			if (bIsDragSelecting)
			{
				ForEachIntersectingLoaderAdapters(GetWorldPartition(), SelectBoxGridSnapped, [&LoaderAdaptersToSelect](UObject* AdapterObject)
				{
					LoaderAdaptersToSelect.Add(AdapterObject);
				});
			}
			else if (HoveredLoaderInterface.IsValid())
			{
				if (MouseEvent.IsControlDown() && SelectedLoaderInterfaces.Contains(HoveredLoaderInterface))
				{
					SelectedLoaderInterfaces.Remove(HoveredLoaderInterface);
				}
				else
				{
					LoaderAdaptersToSelect.Add(HoveredLoaderInterface);
				}
			}

			if (MouseEvent.IsControlDown())
			{
				SelectedLoaderInterfaces.Append(LoaderAdaptersToSelect);
			}
			else
			{
				SelectedLoaderInterfaces = MoveTemp(LoaderAdaptersToSelect);
			}

			SelectBox = SelectBoxGridSnapped;

			bIsDragSelecting = false;
		}

		if (bIsRightMouseButtonEffecting)
		{
			bIsPanning = false;
		}

		if (bIsMiddleMouseButtonEffecting)
		{
			bIsMeasuring = false;
		}

		if (HasMouseCapture() && !bIsDragSelecting && !bIsPanning && !bIsMeasuring)
		{
			ReplyState.ReleaseMouseCapture();
		}

		return ReplyState;
	}

	return FReply::Unhandled();
}

FReply SWorldPartitionEditorGrid2D::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (IsInteractive())
	{
		if (InMouseEvent.IsShiftDown())
		{
			if (!GetDefault<UWorldPartitionEditorSettings>()->GetDisablePIE())
			{
				PlayFromHere();
			}
		}
		else
		{
			MoveCameraHere();

			if (InMouseEvent.IsControlDown() && GetDefault<UWorldPartitionEditorSettings>()->GetEnableLoadingInEditor())
			{
				LoadFromHere();
			}
		}
	}
	return FReply::Handled();
}

FReply SWorldPartitionEditorGrid2D::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const FVector2D CursorDelta = MouseEvent.GetCursorDelta();

	MouseCursorPos = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	MouseCursorPosWorld = ScreenToWorld.TransformPoint(MouseCursorPos);

	if (HasMouseCapture())
	{
		TotalMouseDelta += CursorDelta.Size();

		const bool bIsRightMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton);
		const bool bIsLeftMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
		const bool bIsMiddleMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton);
		const bool bIsDragTrigger = IsInteractive() && (TotalMouseDelta > FSlateApplication::Get().GetDragTriggerDistance());

		if (bIsMiddleMouseButtonDown)
		{
			if (!bIsMeasuring && bIsDragTrigger)
			{
				bIsMeasuring = true;
			}

			if (bIsMeasuring)
			{
				MeasureEnd = MouseCursorPosWorld;
			}
		}

		if (bIsLeftMouseButtonDown)
		{
			if (!bIsDragSelecting && bIsDragTrigger)
			{
				bIsDragSelecting = true;
			}

			if (bIsDragSelecting)
			{
				SelectionEnd = MouseCursorPosWorld;
				UpdateSelectionBox(MouseEvent.IsShiftDown());
				return FReply::Handled();
			}
		}

		if (bIsRightMouseButtonDown && !bIsDragSelecting)
		{
			if (!bIsPanning && bIsDragTrigger)
			{
				bIsPanning = true;
				LastMouseCursorPosWorldDrag = MouseCursorPosWorld;
			}

			if (bIsPanning)
			{
				Trans += (MouseCursorPosWorld - LastMouseCursorPosWorldDrag);
				UpdateTransform();
				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

FReply SWorldPartitionEditorGrid2D::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (IsInteractive())
	{
		FVector2D MousePosLocalSpace = MouseCursorPos - MyGeometry.GetLocalSize() * 0.5f;
		FVector2D P0 = MousePosLocalSpace / Scale;
		float Delta = 1.0f + FMath::Abs(MouseEvent.GetWheelDelta() / 8.0f);
		Scale = FMath::Clamp(Scale * (MouseEvent.GetWheelDelta() > 0 ? Delta : (1.0f / Delta)), 0.00000001f, 10.0f);
		FVector2D P1 = MousePosLocalSpace / Scale;
		Trans += (P1 - P0);
		UpdateTransform();
	}
	return FReply::Handled();
}

FCursorReply SWorldPartitionEditorGrid2D::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	return FCursorReply::Cursor(bIsPanning ? EMouseCursor::GrabHandClosed : EMouseCursor::Default);
}

int32 SWorldPartitionEditorGrid2D::PaintGrid(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	// Draw grid lines
	TArray<FVector2D> LinePoints;
	LinePoints.SetNum(2);

	const FVector2D ScreenWorldOrigin = WorldToScreen.TransformPoint(FVector2D(0, 0));
	
	// World Y-axis
	if (ScreenWorldOrigin.X > ScreenRect.Min.X && ScreenWorldOrigin.X < ScreenRect.Max.X)
	{
		LinePoints[0] = FVector2D(ScreenWorldOrigin.X, ScreenRect.Min.Y);
		LinePoints[1] = FVector2D(ScreenWorldOrigin.X, ScreenRect.Max.Y);

		FLinearColor YAxisColor = AxisDisplayInfo::GetAxisColor(EAxisList::Y);
		YAxisColor.A = 0.4f;
		
		FSlateDrawElement::MakeLines(
			OutDrawElements, 
			LayerId, 
			AllottedGeometry.ToPaintGeometry(), 
			LinePoints, 
			ESlateDrawEffect::None, 
			YAxisColor, 
			true, 
			2.0f
		);
	}

	// World X-axis
	if (ScreenWorldOrigin.Y > ScreenRect.Min.Y && ScreenWorldOrigin.Y < ScreenRect.Max.Y)
	{
		LinePoints[0] = FVector2D(ScreenRect.Min.X, ScreenWorldOrigin.Y);
		LinePoints[1] = FVector2D(ScreenRect.Max.X, ScreenWorldOrigin.Y);

		FLinearColor XAxisColor = AxisDisplayInfo::GetAxisColor(EAxisList::X);
		XAxisColor.A = 0.4f;
		
		FSlateDrawElement::MakeLines(
			OutDrawElements, 
			LayerId, 
			AllottedGeometry.ToPaintGeometry(), 
			LinePoints, 
			ESlateDrawEffect::None, 
			XAxisColor, 
			true, 
			2.0f
		);
	}

	return LayerId + 1;
}

bool SWorldPartitionEditorGrid2D::ShouldShowActorBounds(AActor* InActor) const
{
	return bShowActors && (bShowHLODActors || !InActor->IsA<AWorldPartitionHLOD>());
}

bool SWorldPartitionEditorGrid2D::ShouldShowActorBounds(FWorldPartitionActorDescInstance* ActorDescInstance) const
{
	return bShowActors && (bShowHLODActors || !ActorDescInstance->GetActorNativeClass()->IsChildOf<AWorldPartitionHLOD>());
}

void SWorldPartitionEditorGrid2D::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	FWeightedMovingAverageScope ProfileMeanValue(TickTime);

	const FBox2D ViewRect(FVector2D(ForceInitToZero), AllottedGeometry.GetLocalSize());
	const FBox2D WorldViewRect(ScreenToWorld.TransformPoint(ViewRect.Min), ScreenToWorld.TransformPoint(ViewRect.Max));
	const FBox ViewRectWorld(FVector(WorldViewRect.Min.X, WorldViewRect.Min.Y, -HALF_WORLD_MAX), FVector(WorldViewRect.Max.X, WorldViewRect.Max.Y, HALF_WORLD_MAX));

	ShownActorGuids.Reset();
	DirtyActorGuids.Reset();
	SelectedActorGuids.Reset();
	ShownLoaderInterfaces.Reset(); 
	
	for (UWorldPartitionEditorLoaderAdapter* EditorLoaderAdapter : GetRegisteredEditorLoaderAdapters(GetWorldPartition()))
	{
		check(EditorLoaderAdapter);
		IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = EditorLoaderAdapter->GetLoaderAdapter();
		check(LoaderAdapter);

		if (LoaderAdapter->GetBoundingBox().IsSet() && LoaderAdapter->GetBoundingBox()->IntersectXY(ViewRectWorld))
		{
			ShownLoaderInterfaces.Add(EditorLoaderAdapter);
		}
	}

	UWorldPartitionEditorHash::FForEachIntersectingActorParams ForEachIntersectingActorParams = UWorldPartitionEditorHash::FForEachIntersectingActorParams()
		.SetMinimumBox(FBox(FVector::ZeroVector, FVector(GetSelectionSnap())));

	if (ViewRectWorld != ViewRectWorldCache)
	{
		ShownActorGuidsCache.Reset();
		ShownLoaderInterfacesCache.Reset();

		GetWorldPartition()->EditorHash->ForEachIntersectingActor(ViewRectWorld, [&](FWorldPartitionActorDescInstance* ActorDescInstance)
		{
			if (ShouldShowActorBounds(ActorDescInstance))
			{
				if (ActorDescInstance->IsListedInSceneOutliner() && (!GetWorldPartition()->IsStreamingEnabled() || ActorDescInstance->GetIsSpatiallyLoaded()))
				{
					ShownActorGuidsCache.Add(ActorDescInstance->GetGuid());
				}
			}

			if (ActorDescInstance->GetActorNativeClass()->ImplementsInterface(UWorldPartitionActorLoaderInterface::StaticClass()))
			{
				if (AActor* Actor = ActorDescInstance->GetActor())
				{
					if (IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = Cast<IWorldPartitionActorLoaderInterface>(Actor)->GetLoaderAdapter())
					{
						ShownLoaderInterfacesCache.Add(Actor);
					}
				}
			}
		}, ForEachIntersectingActorParams);

		ViewRectWorldCache = ViewRectWorld;
	}

	ShownActorGuids.Append(ShownActorGuidsCache);
	ShownLoaderInterfaces.Append(ShownLoaderInterfacesCache);

	// Also include dirty actors so we can display updated bounds.
	if (ExternalDirtyActorsTracker.IsValid())
	{
		for (auto& [WeakActor, ActorGuid] : ExternalDirtyActorsTracker->GetDirtyActors())
		{
			if (WeakActor.IsValid())
			{
				DirtyActorGuids.Add(ActorGuid);

				// Also include transient actor loader adapters that might have been spawned by blutilities, etc. Since these actors can't be saved because they are transient,
				// they will never get an actor descriptor so they will never appear in the world partition editor. Also include unsaved, newly created actors for convenience.
				if (WeakActor->Implements<UWorldPartitionActorLoaderInterface>())
				{
					if (IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = Cast<IWorldPartitionActorLoaderInterface>(WeakActor.Get())->GetLoaderAdapter())
					{
						ShownLoaderInterfaces.Add(WeakActor.Get());
					}
				}

				if (ShouldShowActorBounds(WeakActor.Get()))
				{
					ShownActorGuids.Add(ActorGuid);
				}
			}
		}
	}

	FLoaderInterfaceSet LastHoveredLoaderInterfaces = MoveTemp(HoveredLoaderInterfaces);
	for (const FLoaderInterface& LoaderInterface : ShownLoaderInterfaces)
	{
		if (!LoaderInterface.IsValid())
		{
			continue;
		}

		const IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = LoaderInterface->GetLoaderAdapter();

		if (LoaderAdapter->GetBoundingBox().IsSet())
		{
			if (IsBoundsHovered(MouseCursorPosWorld, *LoaderAdapter->GetBoundingBox()))
			{
				HoveredLoaderInterfaces.Add(LoaderInterface);
			}
		}
	}

	FLoaderInterfaceSet EnteredHoveredLoaderInterfaces = HoveredLoaderInterfaces.Difference(LastHoveredLoaderInterfaces);
	FLoaderInterfaceSet ExitedHoveredLoaderInterfaces = LastHoveredLoaderInterfaces.Difference(HoveredLoaderInterfaces);

	if (EnteredHoveredLoaderInterfaces.Num())
	{
		if (HoveredLoaderInterface.IsValid())
		{
			HoveredLoaderInterfacesStack.Push(HoveredLoaderInterface);
		}

		HoveredLoaderInterface = *FLoaderInterfaceSet::TIterator(EnteredHoveredLoaderInterfaces);
	}
	else if (ExitedHoveredLoaderInterfaces.Num())
	{
		if (ExitedHoveredLoaderInterfaces.Contains(HoveredLoaderInterface))
		{
			HoveredLoaderInterfaces.Remove(HoveredLoaderInterface);
			HoveredLoaderInterface = nullptr;

			// Go back in the hovered stack if possible
			while (!HoveredLoaderInterfacesStack.IsEmpty())
			{
				if (FLoaderInterface StackedHoveredLoaderInterface = HoveredLoaderInterfacesStack.Pop(); StackedHoveredLoaderInterface.IsValid())
				{
					IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = StackedHoveredLoaderInterface->GetLoaderAdapter();
					if (LoaderAdapter && IsBoundsHovered(MouseCursorPosWorld, *StackedHoveredLoaderInterface->GetLoaderAdapter()->GetBoundingBox()))
					{
						HoveredLoaderInterface = StackedHoveredLoaderInterface;
						break;
					}
				}
			}

			// Last resort, take the first one in the list
			if (!HoveredLoaderInterface.IsValid() && HoveredLoaderInterfaces.Num())
			{
				HoveredLoaderInterface = *FLoaderInterfaceSet::TIterator(HoveredLoaderInterfaces);
			}
		}
	}

	// Include selected actors
	for (FSelectionIterator It = GEditor->GetSelectedActorIterator(); It; ++It)
	{
		if (AActor* Actor = Cast<AActor>(*It))
		{
			ShownActorGuids.Add(Actor->GetActorGuid());
			SelectedActorGuids.Add(Actor->GetActorGuid());
		}
	}

	// Include selected actor descriptors
	UWorldPartitionSubsystem* WorldPartitionSubsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(GetWorld());
	for (const FWorldPartitionHandle& SelectedActorHandle : WorldPartitionSubsystem->SelectedActorHandles)
	{
		ShownActorGuids.Add(SelectedActorHandle->GetGuid());
		SelectedActorGuids.Add(SelectedActorHandle->GetGuid());
	}
}

uint32 SWorldPartitionEditorGrid2D::PaintActors(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, uint32 LayerId) const
{
	const UWorldPartition* ThisWorldPartition = GetWorldPartition();
	const FBox2D ViewRect(FVector2D(ForceInitToZero), AllottedGeometry.GetLocalSize());
	const FBox2D WorldViewRect(ScreenToWorld.TransformPoint(ViewRect.Min), ScreenToWorld.TransformPoint(ViewRect.Max));
	const FBox ViewRectWorld(FVector(WorldViewRect.Min.X, WorldViewRect.Min.Y, -HALF_WORLD_MAX), FVector(WorldViewRect.Max.X, WorldViewRect.Max.Y, HALF_WORLD_MAX));
	const UWorldPartitionSubsystem* WorldPartitionSubsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(GetWorld());

	const IWorldPartitionActorLoaderInterface::ILoaderAdapter* LocalHoveredLoaderAdapter = nullptr;
	if (IWorldPartitionActorLoaderInterface* HoveredLoaderAdapterInterface = Cast<IWorldPartitionActorLoaderInterface>(HoveredLoaderInterface.Get()))
	{
		LocalHoveredLoaderAdapter = HoveredLoaderInterface->GetLoaderAdapter();
	}

	struct FActorBoundsDesc
	{
		FActorBoundsDesc(const FWorldPartitionActorDescInstance* InActorDescInstance, const AActor* InActor)
		{
			if (InActor)
			{
				Guid = InActor->GetActorGuid();
				Label = *InActor->GetActorLabel(false);
				FBox RuntimeActorBounds;
				InActor->GetStreamingBounds(RuntimeActorBounds, ActorBounds);
				DescBounds = InActorDescInstance ? InActorDescInstance->GetEditorBounds() : ActorBounds;
				bIsSpatiallyLoaded = InActor->GetIsSpatiallyLoaded();
			}
			else if (InActorDescInstance)
			{
				Guid = InActorDescInstance->GetGuid();
				Label = InActorDescInstance->GetActorLabel();
				ActorBounds = DescBounds = InActorDescInstance->GetEditorBounds();
				bIsSpatiallyLoaded = InActorDescInstance->GetIsSpatiallyLoaded();
			}
		}

		FGuid Guid;
		FName Label;
		FBox DescBounds;
		FBox ActorBounds;		
		bool bIsSpatiallyLoaded;
	};

	TArray<FActorBoundsDesc> ActorBoundsDescs;
	ActorBoundsDescs.Reserve(ShownActorGuids.Num());
	
	for (const FGuid& ActorGuid : ShownActorGuids)
	{
		if (const FWorldPartitionActorDescInstance* ActorDescInstance = ThisWorldPartition->GetActorDescInstance(ActorGuid))
		{
			ActorBoundsDescs.Emplace(ActorDescInstance, DirtyActorGuids.Contains(ActorGuid) ? ActorDescInstance->GetActor(false) : nullptr);
		}
	}

	if (ExternalDirtyActorsTracker.IsValid())
	{
		for (auto& [WeakActor, ActorGuid] : ExternalDirtyActorsTracker->GetDirtyActors())
		{
			if (WeakActor.IsValid() && (ShouldShowActorBounds(WeakActor.Get()) || SelectedActorGuids.Contains(ActorGuid)) && !ThisWorldPartition->GetActorDescInstance(ActorGuid))
			{
				ActorBoundsDescs.Emplace(nullptr, WeakActor.Get());
			}
		}
	}

	auto DrawActorLabel = [this, &OutDrawElements, &LayerId, &AllottedGeometry](const FString& Label, const FVector2D& Pos, const FPaintGeometry& Geometry, const FLinearColor& Color, const FSlateFontInfo& Font, bool bShowBackground)
	{
		const FVector2D LabelTextSize = FSlateApplication::Get().GetRenderer()->GetFontMeasureService()->Measure(Label, Font);

		if (LabelTextSize.X > 0)
		{
			const FVector2D BackgroundGrowSize(2.0f);
			const FVector2D LabelTextPos = Pos - LabelTextSize * 0.5f;

			if (bShowBackground)
			{
				const FSlateColorBrush BackgroundBrush(USlateThemeManager::Get().GetColor(EStyleColor::Black));
				const FLinearColor LabelBackgroundColor = USlateThemeManager::Get().GetColor(EStyleColor::Black).CopyWithNewOpacity(Color.A * 0.1f);

				FSlateDrawElement::MakeBox(
					OutDrawElements,
					++LayerId,
					AllottedGeometry.ToPaintGeometry(LabelTextSize + BackgroundGrowSize, FSlateLayoutTransform(LabelTextPos - BackgroundGrowSize)),
					&BackgroundBrush,
					ESlateDrawEffect::None,
					LabelBackgroundColor
				);
			}

			LayerId = DrawTextLabel(
				OutDrawElements, 
				++LayerId, 
				AllottedGeometry, 
				Label, 
				Pos, 
				Color, 
				Font
			);
		}
	};

	if (bShowLoadingRegions && ShownLoaderInterfaces.Num())
	{
		TArray<FVector2D> LinePoints;
		LinePoints.SetNum(5);

		for (const FLoaderInterface& LoaderInterface : ShownLoaderInterfaces)
		{
			if (!LoaderInterface.IsValid())
			{
				continue;
			}

			const IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = LoaderInterface->GetLoaderAdapter();

			if (LoaderAdapter->GetBoundingBox().IsSet())
			{
				const FBox AdapterBounds = *LoaderAdapter->GetBoundingBox();

				FVector Origin, Extent;
				AdapterBounds.GetCenterAndExtents(Origin, Extent);

				const FVector2D TopLeftW = FVector2D(Origin - Extent);
				const FVector2D BottomRightW = FVector2D(Origin + Extent);
				const FVector2D TopRightW = FVector2D(BottomRightW.X, TopLeftW.Y);
				const FVector2D BottomLeftW = FVector2D(TopLeftW.X, BottomRightW.Y);

				const FVector2D TopLeft = WorldToScreen.TransformPoint(TopLeftW);
				const FVector2D BottomRight = WorldToScreen.TransformPoint(BottomRightW);
				const FVector2D TopRight = WorldToScreen.TransformPoint(TopRightW);
				const FVector2D BottomLeft = WorldToScreen.TransformPoint(BottomLeftW);

				const FBox2D ActorViewBox(TopLeft, BottomRight);

				const float FullScreenColorGradient = FMath::Min(ViewRect.GetArea() / ActorViewBox.GetArea(), 1.0f);

				if (FullScreenColorGradient > 0.0f)
				{
					const float MinimumAreaCull = 32.0f;
					const float AreaFadeDistance = 128.0f;
					if ((Extent.Size2D() < KINDA_SMALL_NUMBER) || (ActorViewBox.GetArea() > MinimumAreaCull))
					{
						const FPaintGeometry ActorGeometry = AllottedGeometry.ToPaintGeometry(BottomRight - TopLeft, FSlateLayoutTransform(TopLeft));
						const float LoaderColorGradient = FMath::Min((ActorViewBox.GetArea() - MinimumAreaCull) / AreaFadeDistance, 1.0f);
						const FLinearColor LoaderColor = LoaderAdapter->GetColor().IsSet() ? *LoaderAdapter->GetColor() : USlateThemeManager::Get().GetColor(EStyleColor::White);
						const bool bIsLocalHovered = LocalHoveredLoaderAdapter == LoaderAdapter;

						// Highlight
						{
							const FSlateColorBrush LoadedBrush(USlateThemeManager::Get().GetColor(EStyleColor::White));
							const FSlateColorBrush UnloadedBrush(USlateThemeManager::Get().GetColor(EStyleColor::AccentGray));
							const FLinearColor LoadedColor(LoaderColor.R, LoaderColor.G, LoaderColor.B, 0.23f * LoaderColorGradient * FullScreenColorGradient);
							const FLinearColor UnloadedColor(LoaderColor.R * 0.15f, LoaderColor.G * 0.15f, LoaderColor.B * 0.15f, 0.25f * LoaderColorGradient * FullScreenColorGradient);

							// Clip to minimap
							F2DRectBooleanSubtract LoaderInterfaceHighlight(ActorViewBox);

							if (!bIsLocalHovered)
							{
								if (WorldMiniMapBounds.bIsValid)
								{
									const FBox2D MinimapBounds(
										WorldToScreen.TransformPoint(WorldMiniMapBounds.Min),
										WorldToScreen.TransformPoint(WorldMiniMapBounds.Max)
									);
	
									LoaderInterfaceHighlight.SubRect(MinimapBounds);
								}
							}

							for (const FBox2D& Rect : LoaderInterfaceHighlight.GetRects())
							{
								const FPaintGeometry RectAreaGeometry = AllottedGeometry.ToPaintGeometry(
									Rect.Max - Rect.Min,
									FSlateLayoutTransform(Rect.Min)
								);

								FSlateDrawElement::MakeBox(
									OutDrawElements,
									++LayerId,
									RectAreaGeometry,
									LoaderAdapter->IsLoaded() ? &LoadedBrush : &UnloadedBrush,
									ESlateDrawEffect::None,
									LoaderAdapter->IsLoaded() ? LoadedColor : UnloadedColor
								);
							}
						}

						// Outline
						{
							const bool bIsSelected = SelectedLoaderInterfaces.Contains(LoaderInterface);
							const bool bIsInsideSelection = (SelectBoxGridSnapped.GetVolume() > 0) && SelectBoxGridSnapped.Intersect(AdapterBounds);
							
							float OutlineThickness = 2.0f;
							FLinearColor OutlineColor = USlateThemeManager::Get().GetColor(EStyleColor::AccentGray);

							if (bIsSelected)
							{
								OutlineThickness = 4.0f;
								OutlineColor = USlateThemeManager::Get().GetColor(EStyleColor::Primary);
							}
							else if (bIsInsideSelection)
							{
								OutlineThickness = 4.0f;
								OutlineColor = USlateThemeManager::Get().GetColor(EStyleColor::Primary) * 0.5f;
							}
							else if (bIsLocalHovered)
							{
								OutlineThickness = 4.0f;
								OutlineColor = USlateThemeManager::Get().GetColor(EStyleColor::White);
							}

							LinePoints[0] = TopLeft;
							LinePoints[1] = TopRight;
							LinePoints[2] = BottomRight;
							LinePoints[3] = BottomLeft;
							LinePoints[4] = TopLeft;

							FSlateDrawElement::MakeLines
							(
								OutDrawElements,
								++LayerId,
								AllottedGeometry.ToPaintGeometry(),
								LinePoints,
								ESlateDrawEffect::None,
								OutlineColor,
								true,
								OutlineThickness
							);
						}

						// Label
						{
							const FString ActorLabel = *LoaderAdapter->GetLabel();
							const FLinearColor LabelColor = USlateThemeManager::Get().GetColor(EStyleColor::White).CopyWithNewOpacity(LoaderColorGradient * FullScreenColorGradient);
							DrawActorLabel(ActorLabel, ActorViewBox.GetCenter(), ActorGeometry, LabelColor, SmallLayoutFont, false);
						}
					}
				}
			}
		}
	}

	if (ActorBoundsDescs.Num())
	{
		TArray<FVector2D> LinePoints;
		LinePoints.SetNum(5);

		bool bIsBoundsEdgeHoveredFound = false;
		for (const FActorBoundsDesc& ActorBoundsDesc : ActorBoundsDescs)
		{
			auto ShowActorBox = [this, &AllottedGeometry, &OutDrawElements, &LayerId, &bIsBoundsEdgeHoveredFound, &DrawActorLabel](const FBox& ActorBounds, bool bIsSelected, bool bIsSpatiallyLoaded, const FName ActorLabel)
			{
				FVector Origin, Extent;
				ActorBounds.GetCenterAndExtents(Origin, Extent);

				FVector2D TopLeftW = FVector2D(Origin - Extent);
				FVector2D BottomRightW = FVector2D(Origin + Extent);
				FVector2D TopRightW = FVector2D(BottomRightW.X, TopLeftW.Y);
				FVector2D BottomLeftW = FVector2D(TopLeftW.X, BottomRightW.Y);

				FVector2D TopLeft = WorldToScreen.TransformPoint(TopLeftW);
				FVector2D BottomRight = WorldToScreen.TransformPoint(BottomRightW);
				FVector2D TopRight = WorldToScreen.TransformPoint(TopRightW);
				FVector2D BottomLeft = WorldToScreen.TransformPoint(BottomLeftW);

				FBox2D ActorViewBox(TopLeft, BottomRight);

				const float MinimumAreaCull = 32.0f;
				const float AreaFadeDistance = 128.0f;
				if ((Extent.Size2D() < KINDA_SMALL_NUMBER) || (ActorViewBox.GetArea() > MinimumAreaCull))
				{
					const FPaintGeometry ActorGeometry = AllottedGeometry.ToPaintGeometry(BottomRight - TopLeft, FSlateLayoutTransform(TopLeft));
					const FPaintGeometry ActorGeometryShadow = AllottedGeometry.ToPaintGeometry(BottomRight - TopLeft + FVector2D::One() * 2.0, FSlateLayoutTransform(TopLeft - FVector2D::One()));
					const bool bIsBoundsEdgeHovered = !bIsBoundsEdgeHoveredFound && IsBoundsEdgeHovered(MouseCursorPos, ActorViewBox, 10.0f);
					const float ActorColorGradient = FMath::Cube(FMath::Clamp((ActorViewBox.GetArea() - MinimumAreaCull) / AreaFadeDistance, 0.0f, 1.0f));
					const float ActorBrightness = bIsSpatiallyLoaded ? 1.0f : 0.3f;

					// Only draw the first edge bounds hovered
					bIsBoundsEdgeHoveredFound |= bIsBoundsEdgeHovered;
				
					FLinearColor ActorColor(ActorBrightness, ActorBrightness, ActorBrightness, bIsBoundsEdgeHovered ? 1.0f : ActorColorGradient);

					if (bIsSelected || bIsBoundsEdgeHovered)
					{
						if (!ActorLabel.IsNone())
						{
							const FLinearColor LabelColor = USlateThemeManager::Get().GetColor(EStyleColor::AccentOrange).CopyWithNewOpacity(ActorColorGradient);
							DrawActorLabel(ActorLabel.ToString(), !bIsSelected ? MouseCursorPos : ActorViewBox.GetCenter(), ActorGeometry, LabelColor, SmallLayoutFont, true);
						}

						ActorColor = USlateThemeManager::Get().GetColor(EStyleColor::AccentOrange).CopyWithNewOpacity(ActorColor.A);
					}
					else if ((SelectBoxGridSnapped.GetVolume() > 0) && SelectBoxGridSnapped.Intersect(ActorBounds))
					{
						ActorColor = USlateThemeManager::Get().GetColor(EStyleColor::White).CopyWithNewOpacity(ActorColor.A);
					}

					FSlateDrawElement::MakeBox(
						OutDrawElements,
						++LayerId,
						ActorGeometryShadow,
						FAppStyle::GetBrush(TEXT("Border")),
						ESlateDrawEffect::None,
						USlateThemeManager::Get().GetColor(EStyleColor::Black).CopyWithNewOpacity(ActorColor.A)
					);

					FSlateDrawElement::MakeBox(
						OutDrawElements,
						++LayerId,
						ActorGeometry,
						FAppStyle::GetBrush(TEXT("Border")),
						ESlateDrawEffect::None,
						ActorColor
					);
				}
			};

			if (ActorBoundsDesc.Guid.IsValid())
			{			
				const FBox ActorBounds = ActorBoundsDesc.ActorBounds;
				const bool bIsSelected = SelectedActorGuids.Contains(ActorBoundsDesc.Guid);
				const bool bIsSpatiallyLoaded = ActorBoundsDesc.bIsSpatiallyLoaded;
				const FName ActorLabel = ActorBoundsDesc.Label;

				if (bIsSelected)
				{
					const FBox ActorDescBounds = ActorBoundsDesc.DescBounds;
					if (!ActorDescBounds.Equals(ActorBounds, 1.0f))
					{
						ShowActorBox(ActorDescBounds, false, bIsSpatiallyLoaded, ActorLabel);
					}
				}

				ShowActorBox(ActorBounds, bIsSelected, bIsSpatiallyLoaded, ActorLabel);
			}
		};
	}

	return LayerId + 1;
}

uint32 SWorldPartitionEditorGrid2D::PaintTextInfo(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, uint32 LayerId) const
{
	const float	ScaleRulerLength = 100.f; // pixels
	TArray<FVector2D> LinePoints;
	LinePoints.Add(FVector2D::ZeroVector);
	LinePoints.Add(FVector2D::ZeroVector + FVector2D(ScaleRulerLength, 0.f));
	
	int InX = AllottedGeometry.GetLocalSize().X - ScaleRulerLength;
	int InY = AllottedGeometry.GetLocalSize().Y - 8;

	FSlateDrawElement::MakeLines(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToOffsetPaintGeometry(FVector2D(InX, InY)),
		LinePoints,
		ESlateDrawEffect::None,
		USlateThemeManager::Get().GetColor(EStyleColor::White));

	const float UnitsInRuler = ScaleRulerLength/Scale + 0.05f;// Pixels to world units (+0.05f to accommodate for %.2f)
	const int32 UnitsInMeter = 100;
	const int32 UnitsInKilometer = UnitsInMeter*1000;
	
	FString RulerText = (UnitsInRuler >= UnitsInKilometer) ? FString::Printf(TEXT("%.2f km"), UnitsInRuler / UnitsInKilometer) : FString::Printf(TEXT("%.2f m"), UnitsInRuler / UnitsInMeter);
	if (TextRulerWidget.IsValid())
	{
		TextRulerWidget->SetText(FText::FromString(RulerText));
	}

	// Show world bounds
	const FBox WorldBounds = GetWorldPartition()->GetRuntimeWorldBounds();
	const FVector WorldBoundsExtentInKM = (WorldBounds.GetExtent() * 2.0f) / 100000.0f;
	if (TextWorldBoundsInKMWidget.IsValid())
	{
		TextWorldBoundsInKMWidget->SetText(FText::FromString(FString::Printf(TEXT("%.2fx%.2fx%.2f km"), WorldBoundsExtentInKM.X, WorldBoundsExtentInKM.Y, WorldBoundsExtentInKM.Z)));
	}

	// Show profiling
	if (GShowEditorProfilingStats)
	{
		RulerText = FString::Printf(TEXT("TickTime=%s"), *FPlatformTime::PrettyTime(TickTime));
		FSlateDrawElement::MakeText(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToOffsetPaintGeometry(FVector2D(10, 107)),
			RulerText,
			FAppStyle::GetFontStyle("NormalFont"),
			ESlateDrawEffect::None,
			USlateThemeManager::Get().GetColor(EStyleColor::AccentGray));

		const FVector2D TextSize = FSlateApplication::Get().GetRenderer()->GetFontMeasureService()->Measure(RulerText, FAppStyle::GetFontStyle("NormalFont"));

		RulerText = FString::Printf(TEXT("PaintTime=%s"), *FPlatformTime::PrettyTime(PaintTime));
		FSlateDrawElement::MakeText(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToOffsetPaintGeometry(FVector2D(10, 107 + TextSize.Y + 2)),
			RulerText,
			FAppStyle::GetFontStyle("NormalFont"),
			ESlateDrawEffect::None,
			USlateThemeManager::Get().GetColor(EStyleColor::AccentGray));
	}

	if (bShowMouseCoords)
	{
		const FString MouseCursorPosWorldText = FString::Printf(TEXT("(%.2f, %.2f)"), MouseCursorPosWorld.X, MouseCursorPosWorld.Y);
		const FVector2D MouseCursorPosWorldTextSize = FSlateApplication::Get().GetRenderer()->GetFontMeasureService()->Measure(MouseCursorPosWorldText, FAppStyle::GetFontStyle("NormalFont"));
		const FVector2D CurrentCursorSize = FSlateApplication::Get().GetCursorSize();
		const float OneThirdCursorY = (CurrentCursorSize.Y * 0.5) - (MouseCursorPosWorldTextSize.Y * 0.3);

		FSlateDrawElement::MakeText(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToOffsetPaintGeometry(FVector2D(MouseCursorPos.X + (CurrentCursorSize.X * 0.5) + 6, MouseCursorPos.Y + OneThirdCursorY)),
			MouseCursorPosWorldText,
			FAppStyle::GetFontStyle("NormalFont"),
			ESlateDrawEffect::None,
			USlateThemeManager::Get().GetColor(EStyleColor::White));
	}
		
	return LayerId + 1;
}

uint32 SWorldPartitionEditorGrid2D::PaintViewer(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, uint32 LayerId) const
{
	auto MakeRotatedBoxWithShadow = [this, &AllottedGeometry, &OutDrawElements, &LayerId](const FVector2D& Location, const FRotator& Rotation, const FSlateBrush* Image, const FLinearColor& Color, const FVector2D& ShadowSize)
	{
		const FVector2D LocalLocation = WorldToScreen.TransformPoint(Location);
		const FPaintGeometry PaintGeometryShadow = AllottedGeometry.ToPaintGeometry(
			Image->ImageSize + ShadowSize,
			FSlateLayoutTransform(LocalLocation - (Image->ImageSize + ShadowSize) * 0.5f)
		);

		FSlateDrawElement::MakeRotatedBox(
			OutDrawElements,
			++LayerId,
			PaintGeometryShadow,
			Image,
			ESlateDrawEffect::None,
			FMath::DegreesToRadians(Rotation.Yaw),
			(Image->ImageSize + ShadowSize) * 0.5f,
			FSlateDrawElement::RelativeToElement,
			USlateThemeManager::Get().GetColor(EStyleColor::Black)
		);

		const FPaintGeometry PaintGeometry = AllottedGeometry.ToPaintGeometry(
			Image->ImageSize,
			FSlateLayoutTransform(LocalLocation - Image->ImageSize * 0.5f)
		);

		FSlateDrawElement::MakeRotatedBox(
			OutDrawElements,
			++LayerId,
			PaintGeometry,
			Image,
			ESlateDrawEffect::None,
			FMath::DegreesToRadians(Rotation.Yaw),
			Image->ImageSize * 0.5f,
			FSlateDrawElement::RelativeToElement,
			Color
		);
	};

	const FVector2D ShadowSize(2, 2);
	const FSlateBrush* CameraImage = FAppStyle::GetBrush(TEXT("WorldPartition.SimulationViewPosition"));

	FVector ObserverPosition;
	FRotator ObserverRotation;
	if (GetObserverView(ObserverPosition, ObserverRotation))
	{
		MakeRotatedBoxWithShadow(FVector2D(ObserverPosition), ObserverRotation, CameraImage, USlateThemeManager::Get().GetColor(EStyleColor::White), ShadowSize);
	}

	FVector PlayerPosition;
	FRotator PlayerRotation;
	if (GetPlayerView(PlayerPosition, PlayerRotation))
	{
		MakeRotatedBoxWithShadow(FVector2D(PlayerPosition), PlayerRotation, CameraImage, USlateThemeManager::Get().GetColor(EStyleColor::AccentOrange), ShadowSize);
	}

	return LayerId + 1;
}

uint32 SWorldPartitionEditorGrid2D::PaintSelection(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, uint32 LayerId) const
{
	if (SelectBoxGridSnapped.IsValid)
	{
		TArray<FVector2D> LinePoints;
		LinePoints.SetNum(5);

		FLinearColor OutlineColor(USlateThemeManager::Get().GetColor(EStyleColor::White));
		float OutlineThickness = 1.0f;

		// Draw snapped box
		{
			FVector2D TopLeftW = FVector2D(SelectBoxGridSnapped.Min);
			FVector2D BottomRightW = FVector2D(SelectBoxGridSnapped.Max);
			FVector2D TopRightW = FVector2D(BottomRightW.X, TopLeftW.Y);
			FVector2D BottomLeftW = FVector2D(TopLeftW.X, BottomRightW.Y);

			FVector2D TopLeft = WorldToScreen.TransformPoint(TopLeftW);
			FVector2D BottomRight = WorldToScreen.TransformPoint(BottomRightW);
			FVector2D TopRight = WorldToScreen.TransformPoint(TopRightW);
			FVector2D BottomLeft = WorldToScreen.TransformPoint(BottomLeftW);

			LinePoints[0] = TopLeft;
			LinePoints[1] = TopRight;
			LinePoints[2] = BottomRight;
			LinePoints[3] = BottomLeft;
			LinePoints[4] = TopLeft;

			{
				FSlateColorBrush CellBrush(USlateThemeManager::Get().GetColor(EStyleColor::White));
				FLinearColor CellColor = USlateThemeManager::Get().GetColor(EStyleColor::White).CopyWithNewOpacity(0.1f);

				FPaintGeometry CellGeometry = AllottedGeometry.ToPaintGeometry(
					BottomRight - TopLeft,
					FSlateLayoutTransform(TopLeft)
				);

				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId,
					CellGeometry,
					&CellBrush,
					ESlateDrawEffect::None,
					CellColor
				);
			}

			FSlateDrawElement::MakeLines(
				OutDrawElements, 
				LayerId, 
				AllottedGeometry.ToPaintGeometry(), 
				LinePoints, 
				ESlateDrawEffect::None, 
				OutlineColor,
				true, 
				OutlineThickness
			);
		}

		// Draw raw box
		{
			FVector2D TopLeftW = FVector2D(SelectBox.Min);
			FVector2D BottomRightW = FVector2D(SelectBox.Max);
			FVector2D TopRightW = FVector2D(BottomRightW.X, TopLeftW.Y);
			FVector2D BottomLeftW = FVector2D(TopLeftW.X, BottomRightW.Y);

			FVector2D TopLeft = WorldToScreen.TransformPoint(TopLeftW);
			FVector2D BottomRight = WorldToScreen.TransformPoint(BottomRightW);
			FVector2D TopRight = WorldToScreen.TransformPoint(TopRightW);
			FVector2D BottomLeft = WorldToScreen.TransformPoint(BottomLeftW);

			LinePoints[0] = TopLeft;
			LinePoints[1] = TopRight;
			LinePoints[2] = BottomRight;
			LinePoints[3] = BottomLeft;
			LinePoints[4] = TopLeft;

			FSlateDrawElement::MakeLines(
				OutDrawElements, 
				LayerId, 
				AllottedGeometry.ToPaintGeometry(), 
				LinePoints, 
				ESlateDrawEffect::None, 
				OutlineColor,
				true, 
				OutlineThickness
			);
		}
	}

	return LayerId + 1;
}

int32 SWorldPartitionEditorGrid2D::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FWeightedMovingAverageScope ProfileMeanValue(PaintTime);

	if (GetWorldPartition())
	{
		const bool bResetView = !ScreenRect.bIsValid;

		ScreenRect = FBox2D(FVector2D(0, 0), AllottedGeometry.GetLocalSize());

		if (bResetView)
		{
			const_cast<SWorldPartitionEditorGrid2D*>(this)->FocusWorld();
		}

		UpdateTransform();

		LayerId = PaintGrid(AllottedGeometry, MyCullingRect, OutDrawElements, ++LayerId);
		LayerId = PaintActors(AllottedGeometry, MyCullingRect, OutDrawElements, ++LayerId);
		LayerId = PaintTextInfo(AllottedGeometry, MyCullingRect, OutDrawElements, ++LayerId);
		LayerId = PaintViewer(AllottedGeometry, MyCullingRect, OutDrawElements, ++LayerId);
		LayerId = PaintSelection(AllottedGeometry, MyCullingRect, OutDrawElements, ++LayerId);
		LayerId = PaintMeasureTool(AllottedGeometry, MyCullingRect, OutDrawElements, ++LayerId);
	}

	return SWorldPartitionEditorGrid::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

int32 SWorldPartitionEditorGrid2D::PaintMinimap(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	// Draw MiniMap image if any
	if (UTexture2D* Texture2D = Cast<UTexture2D>(WorldMiniMapBrush.GetResourceObject()))
	{
		const FBox2D MinimapBounds(
			WorldToScreen.TransformPoint(WorldMiniMapBounds.Min),
			WorldToScreen.TransformPoint(WorldMiniMapBounds.Max)
		);

		const FPaintGeometry WorldImageGeometry = AllottedGeometry.ToPaintGeometry(
			MinimapBounds.GetSize(),
			FSlateLayoutTransform(MinimapBounds.Min)
		);

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			WorldImageGeometry,
			&WorldMiniMapBrush,
			ESlateDrawEffect::None,
			USlateThemeManager::Get().GetColor(EStyleColor::White)
		);

		if (GetWorldPartition()->IsStreamingEnabled())
		{
			F2DRectBooleanSubtract ShadowAreas(MinimapBounds);
			for (const FLoaderInterface& LoaderInterface : ShownLoaderInterfaces)
			{
				if (LoaderInterface.IsValid())
				{
					const IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = LoaderInterface->GetLoaderAdapter();

					if (LoaderAdapter->IsLoaded() && LoaderAdapter->GetBoundingBox().IsSet())
					{
						const FBox AdapterBounds = *LoaderAdapter->GetBoundingBox();

						FVector Origin, Extent;
						AdapterBounds.GetCenterAndExtents(Origin, Extent);

						const FVector2D TopLeftW = FVector2D(Origin - Extent);
						const FVector2D BottomRightW = FVector2D(Origin + Extent);
						const FVector2D TopRightW = FVector2D(BottomRightW.X, TopLeftW.Y);
						const FVector2D BottomLeftW = FVector2D(TopLeftW.X, BottomRightW.Y);

						const FVector2D TopLeft = WorldToScreen.TransformPoint(TopLeftW);
						const FVector2D BottomRight = WorldToScreen.TransformPoint(BottomRightW);
						const FVector2D TopRight = WorldToScreen.TransformPoint(TopRightW);
						const FVector2D BottomLeft = WorldToScreen.TransformPoint(BottomLeftW);

						const FBox2D ActorViewBox(TopLeft, BottomRight);

						ShadowAreas.SubRect(ActorViewBox);
					}
				}
			}

			for (const FBox2D& ShadowArea : ShadowAreas.GetRects())
			{
				const FPaintGeometry ShadowAreaGeometry = AllottedGeometry.ToPaintGeometry(
					ShadowArea.Max - ShadowArea.Min,
					FSlateLayoutTransform(ShadowArea.Min)
				);

				const FSlateColorBrush ShadowdBrush(USlateThemeManager::Get().GetColor(EStyleColor::AccentGray));
				const FLinearColor ShadowColor = USlateThemeManager::Get().GetColor(EStyleColor::Black).CopyWithNewOpacity(MiniMapUnloadedOpacity);

				FSlateDrawElement::MakeBox(
					OutDrawElements,
					++LayerId,
					ShadowAreaGeometry,
					&ShadowdBrush,
					ESlateDrawEffect::None,
					ShadowColor
				);
			}
		}

		// Minimap outline
		{
			const TArray<FVector2D> LinePoints(
			{
				MinimapBounds.Min,
				FVector2D(MinimapBounds.Max.X, MinimapBounds.Min.Y), 
				MinimapBounds.Max, 
				FVector2D(MinimapBounds.Min.X, 	MinimapBounds.Max.Y), 
				MinimapBounds.Min
			});

			FSlateDrawElement::MakeLines
			(
				OutDrawElements,
				++LayerId,
				AllottedGeometry.ToPaintGeometry(),
				LinePoints,
				ESlateDrawEffect::None,
				USlateThemeManager::Get().GetColor(EStyleColor::Black)
			);
		}
	}

	return LayerId;
}

int32 SWorldPartitionEditorGrid2D::PaintMeasureTool(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	// Measure tool
	if (bIsMeasuring)
	{
		TArray<FVector2D> LinePoints;
		LinePoints.SetNum(2);

		const FVector2D MeasureStartScreen = WorldToScreen.TransformPoint(MeasureStart);
		const FVector2D MeasureEndScreen = WorldToScreen.TransformPoint(MeasureEnd);
		LinePoints[0] = MeasureStartScreen;
		LinePoints[1] = MeasureEndScreen;

		FSlateDrawElement::MakeLines(
			OutDrawElements, 
			LayerId, 
			AllottedGeometry.ToPaintGeometry(), 
			LinePoints, 
			ESlateDrawEffect::None, 
			USlateThemeManager::Get().GetColor(EStyleColor::AccentOrange),
			true, 
			2.0f
		);

		DrawTextLabel(
			OutDrawElements, 
			LayerId, 
			AllottedGeometry,
			FString::Printf(TEXT("%d"), (int32)FVector2D::Distance(MeasureStart, MeasureEnd)),
			(MeasureStartScreen + MeasureEndScreen) * 0.5f,
			USlateThemeManager::Get().GetColor(EStyleColor::White),
			SmallLayoutFont
		);
	}

	return LayerId;
}

int32 SWorldPartitionEditorGrid2D::DrawTextLabel(FSlateWindowElementList& OutDrawElements, int32 LayerId, const FGeometry& AllottedGeometry, const FString& Label, const FVector2D& Pos, const FLinearColor& Color, const FSlateFontInfo& Font) const
{
	const FVector2D LabelTextSize = FSlateApplication::Get().GetRenderer()->GetFontMeasureService()->Measure(Label, Font);

	if (LabelTextSize.X > 0)
	{
		const FVector2D LabelTextPos = Pos - LabelTextSize * 0.5f;
		const FLinearColor LabelForegroundColor = USlateThemeManager::Get().GetColor(EStyleColor::White).CopyWithNewOpacity(Color.A);
		const FLinearColor LabelShadowColor = USlateThemeManager::Get().GetColor(EStyleColor::Black).CopyWithNewOpacity(Color.A);

		FSlateDrawElement::MakeText(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(FVector2D(1, 1), FSlateLayoutTransform(LabelTextPos + FVector2D(2, 2))),
			Label,
			Font,
			ESlateDrawEffect::None,
			LabelShadowColor
		);

		FSlateDrawElement::MakeText(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(FVector2D(1,1), FSlateLayoutTransform(LabelTextPos)),
			Label,
			Font,
			ESlateDrawEffect::None,
			LabelForegroundColor
		);
	}

	return LayerId;
}

void SWorldPartitionEditorGrid2D::FocusSelection()
{
	FBox SelectionBox(ForceInit);

	for (FSelectionIterator It(*GEditor->GetSelectedActors()); It; ++It)
	{
		if (AActor* Actor = Cast<AActor>(*It))
		{
			FBox RuntimeBounds;
			FBox EditorBounds;
			Actor->GetStreamingBounds(RuntimeBounds, EditorBounds);
			SelectionBox += EditorBounds;
		}
	}

	UWorldPartitionSubsystem* WorldPartitionSubsystem = UWorld::GetSubsystem<UWorldPartitionSubsystem>(GetWorld());
	for (const FWorldPartitionHandle& SelectedActorHandle : WorldPartitionSubsystem->SelectedActorHandles)
	{
		SelectionBox += SelectedActorHandle->GetEditorBounds();
	}

	if (SelectionBox.IsValid)
	{
		FocusBox(SelectionBox);
	}
}

void SWorldPartitionEditorGrid2D::FocusLoadedRegions()
{
	FBox SelectionBox(ForceInit);

	for (UWorldPartitionEditorLoaderAdapter* EditorLoaderAdapter : GetRegisteredEditorLoaderAdapters(GetWorldPartition()))
	{
		check(EditorLoaderAdapter);
		IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = EditorLoaderAdapter->GetLoaderAdapter();
		check(LoaderAdapter);

		if (LoaderAdapter->GetBoundingBox().IsSet() && LoaderAdapter->IsLoaded())
		{
			SelectionBox += *LoaderAdapter->GetBoundingBox();
		}
	}

	for (TActorIterator<AActor> ActorIt(GetWorld()); ActorIt; ++ActorIt)
	{
		if (AActor* Actor = *ActorIt)
		{
			if (Actor->Implements<UWorldPartitionActorLoaderInterface>())
			{
				if (IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = Cast<IWorldPartitionActorLoaderInterface>(Actor)->GetLoaderAdapter())
				{
					if (LoaderAdapter->GetBoundingBox().IsSet() && LoaderAdapter->IsLoaded())
					{
						SelectionBox += *LoaderAdapter->GetBoundingBox();
					}
				}
			}
		}
	}

	FocusBox(SelectionBox);
}

void SWorldPartitionEditorGrid2D::FocusWorld()
{
	FBox SelectionBox(ForceInit);

	// Override the minimap bounds if world partition minimap volumes exists
	for (TActorIterator<AWorldPartitionMiniMapVolume> It(GetWorld()); It; ++It)
	{
		if (AWorldPartitionMiniMapVolume* WorldPartitionMiniMapVolume = *It)
		{
			SelectionBox += WorldPartitionMiniMapVolume->GetBounds().GetBox();
		}
	}

	if (!SelectionBox.IsValid)
	{
		SelectionBox = GetWorldPartition()->GetRuntimeWorldBounds();
	}

	FocusBox(SelectionBox);
}

void SWorldPartitionEditorGrid2D::FocusBox(const FBox& Box) const
{
	if (ScreenRect.bIsValid)
	{
		const FBox2D Box2D(FVector2D(Box.Min), FVector2D(Box.Max));
		Trans = -FVector2D(Box2D.GetCenter());

		if (Box2D.GetArea() > 0.0f)
		{
			const FVector2D ScreenExtent = ScreenRect.GetExtent();
			const FVector2D SelectExtent = FVector2D(Box2D.GetExtent());
			Scale = (ScreenExtent / SelectExtent).GetMin() * 0.85f;
		}

		UpdateTransform();
	}
}

void SWorldPartitionEditorGrid2D::UpdateTransform() const
{
	FTransform2d T;
	FTransform2d V;

	if (IsFollowPlayerInPIE())
	{
		FVector FollowPosition;
		FRotator FollowRotation;

		if (UWorldPartition::IsSimulating() ? GetObserverView(FollowPosition, FollowRotation) : GetPlayerView(FollowPosition, FollowRotation))
		{
			const FVector2D PlayerExtent = FVector2D(10000);
			T = FTransform2d(1.0f, -FVector2D(FollowPosition));
			V = FTransform2d((ScreenRect.GetExtent() / PlayerExtent).GetMin(), FVector2D(ScreenRect.GetSize().X * 0.5f, ScreenRect.GetSize().Y * 0.5f));
		}
	}
	else
	{
		T = FTransform2d(1.0f, Trans);
		V = FTransform2d(Scale, FVector2D(ScreenRect.GetSize().X * 0.5f, ScreenRect.GetSize().Y * 0.5f));
	}

	WorldToScreen = T.Concatenate(V);
	ScreenToWorld = WorldToScreen.Inverse();
}

void SWorldPartitionEditorGrid2D::UpdateSelectionBox(bool bSnap)
{
	if (GetWorldPartition()->IsStreamingEnabled())
	{
		const FBox2D SelectBox2D(FVector2D::Min(SelectionStart, SelectionEnd), FVector2D::Max(SelectionStart, SelectionEnd));

		if (SelectBox2D.GetArea() > 0.0f)
		{
			const float MinX = SelectBox2D.Min.X;
			const float MinY = SelectBox2D.Min.Y;
			const float MaxX = SelectBox2D.Max.X;
			const float MaxY = SelectBox2D.Max.Y;
			SelectBox = FBox(FVector(MinX, MinY, -HALF_WORLD_MAX), FVector(MaxX, MaxY, HALF_WORLD_MAX));

			const int64 SelectionSnap = bSnap ? GetSelectionSnap() : 1;
			const float SnapMinX = FMath::GridSnap(MinX - SelectionSnap / 2, SelectionSnap);
			const float SnapMinY = FMath::GridSnap(MinY - SelectionSnap / 2, SelectionSnap);
			const float SnapMaxX = FMath::GridSnap(MaxX + SelectionSnap / 2, SelectionSnap);
			const float SnapMaxY = FMath::GridSnap(MaxY + SelectionSnap / 2, SelectionSnap);
			SelectBoxGridSnapped = FBox(FVector(SnapMinX, SnapMinY, -HALF_WORLD_MAX), FVector(SnapMaxX, SnapMaxY, HALF_WORLD_MAX));
		}
	}
}

void SWorldPartitionEditorGrid2D::ClearSelection()
{
	SelectedLoaderInterfaces.Empty();
	SelectBox.Init();
	SelectBoxGridSnapped.Init();
}

#undef LOCTEXT_NAMESPACE