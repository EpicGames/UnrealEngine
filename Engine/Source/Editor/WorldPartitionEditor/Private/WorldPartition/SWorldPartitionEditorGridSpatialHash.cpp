// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartition/SWorldPartitionEditorGridSpatialHash.h"
#include "Framework/Application/SlateApplication.h"
#include "WorldPartition/WorldPartitionEditorPerProjectUserSettings.h"
#include "WorldPartition/WorldPartitionEditorSpatialHash.h"
#include "WorldPartition/WorldPartition.h"
#include "Brushes/SlateColorBrush.h"
#include "Fonts/FontMeasure.h"

#define LOCTEXT_NAMESPACE "WorldPartitionEditor"

static inline FBox2D ToBox2D(const FBox& Box)
{
	return FBox2D(FVector2D(Box.Min), FVector2D(Box.Max));
}

void SWorldPartitionEditorGridSpatialHash::Construct(const FArguments& InArgs)
{
	SWorldPartitionEditorGrid2D::Construct(SWorldPartitionEditorGrid::FArguments().InWorld(InArgs._InWorld));
}

int64 SWorldPartitionEditorGridSpatialHash::GetSelectionSnap() const
{
	float FadeRatio;
	float CellScreenSize;
	return GetSelectionSnap(FadeRatio, CellScreenSize);
}

int64 SWorldPartitionEditorGridSpatialHash::GetSelectionSnap(float& FadeRatio, float& CellScreenSize) const
{
	UWorldPartitionEditorSpatialHash* EditorSpatialHash = (UWorldPartitionEditorSpatialHash*)GetWorldPartition()->EditorHash;
	
	int64 EffectiveCellSize = EditorSpatialHash->CellSize;
	for (;;)
	{
		CellScreenSize = WorldToScreen.TransformVector(FVector2D(EffectiveCellSize, EffectiveCellSize)).GetMax();

		// CellScreenSize can be zero when the window screen rect is too small
		if ((CellScreenSize <= 0.0) || (CellScreenSize > WantedCellScreenSize))
		{
			FadeRatio = FMath::Clamp((CellScreenSize - WantedCellScreenSize) / WantedCellScreenSize, 0.0f, 1.0f);
			break;
		}
	
		EffectiveCellSize *= 2;
	}

	return EffectiveCellSize;
}

int32 SWorldPartitionEditorGridSpatialHash::PaintGrid(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	UWorldPartitionEditorSpatialHash* EditorSpatialHash = (UWorldPartitionEditorSpatialHash*)GetWorldPartition()->EditorHash;

	// Found the best cell size depending on the current zoom
	float FadeRatio;
	float CellScreenSize;
	int64 EffectiveCellSize = GetSelectionSnap(FadeRatio, CellScreenSize);
	
	// Compute visible rect
	const FBox2D ViewRect(FVector2D(ForceInitToZero), AllottedGeometry.GetLocalSize());
	const FBox2D ViewRectWorld(ScreenToWorld.TransformPoint(ViewRect.Min), ScreenToWorld.TransformPoint(ViewRect.Max));

	const FBox VisibleGridRectWorld(
		FVector(
			FMath::Max(FMath::FloorToFloat(ViewRectWorld.Min.X / EffectiveCellSize) * EffectiveCellSize, ViewRectWorld.Min.X),
			FMath::Max(FMath::FloorToFloat(ViewRectWorld.Min.Y / EffectiveCellSize) * EffectiveCellSize, ViewRectWorld.Min.Y),
			FMath::FloorToFloat(EditorSpatialHash->EditorBounds.Min.Z / EffectiveCellSize) * EffectiveCellSize
		),
		FVector(
			FMath::Min(FMath::CeilToFloat(ViewRectWorld.Max.X / EffectiveCellSize) * EffectiveCellSize, ViewRectWorld.Max.X),
			FMath::Min(FMath::CeilToFloat(ViewRectWorld.Max.Y / EffectiveCellSize) * EffectiveCellSize, ViewRectWorld.Max.Y),
			FMath::CeilToFloat(EditorSpatialHash->EditorBounds.Max.Z / EffectiveCellSize) * EffectiveCellSize
		)
	);

	// Paint minimap
	if (bShowMiniMap)
	{
		const FBox VisibleMinimapRectWorld(
			FVector(
				FMath::Max(FMath::FloorToFloat(EditorSpatialHash->RuntimeBounds.Min.X / EditorSpatialHash->CellSize) * EditorSpatialHash->CellSize, ViewRectWorld.Min.X),
				FMath::Max(FMath::FloorToFloat(EditorSpatialHash->RuntimeBounds.Min.Y / EditorSpatialHash->CellSize) * EditorSpatialHash->CellSize, ViewRectWorld.Min.Y),
				FMath::FloorToFloat(EditorSpatialHash->RuntimeBounds.Min.Z / EditorSpatialHash->CellSize) * EditorSpatialHash->CellSize
			),
			FVector(
				FMath::Min(FMath::CeilToFloat(EditorSpatialHash->RuntimeBounds.Max.X / EditorSpatialHash->CellSize) * EditorSpatialHash->CellSize, ViewRectWorld.Max.X),
				FMath::Min(FMath::CeilToFloat(EditorSpatialHash->RuntimeBounds.Max.Y / EditorSpatialHash->CellSize) * EditorSpatialHash->CellSize, ViewRectWorld.Max.Y),
				FMath::CeilToFloat(EditorSpatialHash->RuntimeBounds.Max.Z / EditorSpatialHash->CellSize) * EditorSpatialHash->CellSize
			)
		);

		// Shadow minimap area
		FSlateColorBrush ShadowBrush(FLinearColor::Black);
		FLinearColor ShadowColor(FLinearColor(0.0f, 0.0f, 0.0f, 0.5f));

		FPaintGeometry GridGeometry = AllottedGeometry.ToPaintGeometry(
			WorldToScreen.TransformPoint(FVector2D(VisibleMinimapRectWorld.Max)) - WorldToScreen.TransformPoint(FVector2D(VisibleMinimapRectWorld.Min)),
			FSlateLayoutTransform(WorldToScreen.TransformPoint(FVector2D(VisibleMinimapRectWorld.Min)))
		);

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			GridGeometry,
			&ShadowBrush,
			ESlateDrawEffect::None,
			ShadowColor
		);

		LayerId = PaintMinimap(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
	}

	// Paint grid lines
	if (ToBox2D(VisibleGridRectWorld).GetArea() > 0.0f)
	{
		auto PaintGridLines = [this, &OutDrawElements, &LayerId, &AllottedGeometry](const FBox& VisibleGridRectWorld, int64 EffectiveCellSize, float Thickness, const FLinearColor& Color)
		{
			if (Color.A > 0.0f)
			{
				UE::Math::TIntVector2<int64> TopLeftW(
					FMath::FloorToFloat(VisibleGridRectWorld.Min.X / EffectiveCellSize) * EffectiveCellSize,
					FMath::FloorToFloat(VisibleGridRectWorld.Min.Y / EffectiveCellSize) * EffectiveCellSize
				);

				UE::Math::TIntVector2<int64> BottomRightW(
					FMath::CeilToFloat(VisibleGridRectWorld.Max.X / EffectiveCellSize) * EffectiveCellSize,
					FMath::CeilToFloat(VisibleGridRectWorld.Max.Y / EffectiveCellSize) * EffectiveCellSize
				);

				TArray<FVector2D> LinePoints;
				LinePoints.SetNum(2);

				// Horizontal
				for (int64 i=TopLeftW.Y; i<=BottomRightW.Y; i+=EffectiveCellSize)
				{
					FVector2D LineStartH(TopLeftW.X, i);
					FVector2D LineEndH(BottomRightW.X, i);

					LinePoints[0] = WorldToScreen.TransformPoint(LineStartH);
					LinePoints[1] = WorldToScreen.TransformPoint(LineEndH);

					FSlateDrawElement::MakeLines(
						OutDrawElements, 
						LayerId, 
						AllottedGeometry.ToPaintGeometry(), 
						LinePoints, 
						ESlateDrawEffect::None, 
						Color, 
						false, 
						Thickness
					);
				}

				// Vertical
				for (int64 i=TopLeftW.X; i<=BottomRightW.X; i+=EffectiveCellSize)
				{
					FVector2D LineStartH(i, TopLeftW.Y);
					FVector2D LineEndH(i, BottomRightW.Y);

					LinePoints[0] = WorldToScreen.TransformPoint(LineStartH);
					LinePoints[1] = WorldToScreen.TransformPoint(LineEndH);

					FSlateDrawElement::MakeLines(
						OutDrawElements, LayerId, 
						AllottedGeometry.ToPaintGeometry(), 
						LinePoints, 
						ESlateDrawEffect::None, 
						Color, 
						false, 
						Thickness
					);
				}
			}
		};

		auto PaintGridLinesFaded = [this, &EffectiveCellSize, EditorSpatialHash, &PaintGridLines, &VisibleGridRectWorld, CellScreenSize](int32 Granularity, float Thickness)
		{
			const float Alpha = FMath::Min(CellScreenSize / (WantedCellScreenSize * Granularity), (EffectiveCellSize >= EditorSpatialHash->CellSize * Granularity) ? 1.0f : 0.5f);
			PaintGridLines(VisibleGridRectWorld, EffectiveCellSize / Granularity, Thickness, FLinearColor(0.1f, 0.1f, 0.1f, Alpha * 0.25f));
		};

		if (bShowGrid)
		{
			PaintGridLinesFaded(4, 1.0f);
			PaintGridLinesFaded(2, 1.0f);
			PaintGridLinesFaded(1, 2.0f);
		}

		// Draw coordinates
		if ((EffectiveCellSize == EditorSpatialHash->CellSize) && GetMutableDefault<UWorldPartitionEditorPerProjectUserSettings>()->GetShowCellCoords())
		{
			UE::Math::TIntVector2<int64> TopLeftW(
				FMath::FloorToFloat(VisibleGridRectWorld.Min.X / EffectiveCellSize) * EffectiveCellSize,
				FMath::FloorToFloat(VisibleGridRectWorld.Min.Y / EffectiveCellSize) * EffectiveCellSize
			);

			UE::Math::TIntVector2<int64> BottomRightW(
				FMath::CeilToFloat(VisibleGridRectWorld.Max.X / EffectiveCellSize) * EffectiveCellSize,
				FMath::CeilToFloat(VisibleGridRectWorld.Max.Y / EffectiveCellSize) * EffectiveCellSize
			);

			const TSharedRef<FSlateFontMeasure> FontMeasure = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

			FSlateFontInfo CoordsFont;
			FVector2D DefaultCoordTextSize;
			bool bNeedsGradient = true;
			// Use top-left coordinate as default coord text
			const FString DefaultCoordText(FString::Printf(TEXT("(%lld,%lld)"), TopLeftW.X / EffectiveCellSize, TopLeftW.Y / EffectiveCellSize));
			for(int32 DesiredFontSize = 24; DesiredFontSize >= 8; DesiredFontSize -= 2)
			{
				CoordsFont = FCoreStyle::GetDefaultFontStyle("Bold", DesiredFontSize);
				DefaultCoordTextSize = FontMeasure->Measure(DefaultCoordText, CoordsFont);

				if (CellScreenSize > DefaultCoordTextSize.X)
				{
					bNeedsGradient = (DesiredFontSize == 8);
					break;
				}
			}

			if (CellScreenSize > DefaultCoordTextSize.X)
			{
				static float GradientDistance = WantedCellScreenSize;
				float ColorGradient = bNeedsGradient ? FMath::Min((CellScreenSize - DefaultCoordTextSize.X) / GradientDistance, 1.0f) : 1.0f;
				const FLinearColor CoordTextColor(1.0f, 1.0f, 1.0f, ColorGradient);

				for (int64 y = TopLeftW.Y; y < BottomRightW.Y; y += EffectiveCellSize)
				{
					for (int64 x = TopLeftW.X; x < BottomRightW.X; x += EffectiveCellSize)
					{
						const FString CoordText(FString::Printf(TEXT("(%lld,%lld)"), x / EffectiveCellSize, y / EffectiveCellSize));
						const FVector2D CoordTextSize = FontMeasure->Measure(CoordText, CoordsFont);

						FSlateDrawElement::MakeText(
							OutDrawElements,
							++LayerId,
							AllottedGeometry.ToPaintGeometry(FVector2D(1,1), FSlateLayoutTransform(WorldToScreen.TransformPoint(FVector2D(x + EffectiveCellSize / 2, y + EffectiveCellSize / 2)) - CoordTextSize / 2)),
							FString::Printf(TEXT("(%lld,%lld)"), x / EffectiveCellSize, y / EffectiveCellSize),
							CoordsFont,
							ESlateDrawEffect::None,
							CoordTextColor
						);
					}
				}
			}
		}

		++LayerId;
	}

	return SWorldPartitionEditorGrid2D::PaintGrid(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
}

#undef LOCTEXT_NAMESPACE
