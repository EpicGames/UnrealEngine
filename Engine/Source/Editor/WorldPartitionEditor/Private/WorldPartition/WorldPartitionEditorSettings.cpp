// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionEditorSettings.h"
#include "WorldPartition/WorldPartition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionEditorSettings)

PRAGMA_DISABLE_DEPRECATION_WARNINGS

UWorldPartitionEditorSettings::UWorldPartitionEditorSettings()
{
	CommandletClass = UWorldPartitionConvertCommandlet::StaticClass();
	InstancedFoliageGridSize = 25600;
	MinimapLowQualityWorldUnitsPerPixelThreshold = 12800;
	bEnableLoadingInEditor = true;
	bEnableStreamingGenerationLogOnPIE = false;
	bShowHLODsInEditor = true;
	bShowHLODsOverLoadedRegions = false;
	HLODMinDrawDistance = 0;
	HLODMaxDrawDistance = 0;
	bDisablePIE = false;
	bDisableBugIt = false;
	bAdvancedMode = true;
	bEnableAdvancedHLODSettings = true;
}

UWorldPartitionEditorSettings::FOnWorldPartitionEditorSettingsChanged& UWorldPartitionEditorSettings::OnSettingsChanged()
{
	return GetMutableDefault<UWorldPartitionEditorSettings>()->SettingsChangedDelegate;
}

FName UWorldPartitionEditorSettings::GetEnableLoadingInEditorPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UWorldPartitionEditorSettings, bEnableLoadingInEditor);
}

FName UWorldPartitionEditorSettings::GetEnableAdvancedHLODSettingsPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UWorldPartitionEditorSettings, bEnableAdvancedHLODSettings);
}

TSubclassOf<UWorldPartitionConvertCommandlet> UWorldPartitionEditorSettings::GetCommandletClass() const
{
	return CommandletClass;
}

void UWorldPartitionEditorSettings::SetCommandletClass(const TSubclassOf<UWorldPartitionConvertCommandlet>& InCommandletClass)
{
	if (CommandletClass != InCommandletClass)
	{
		CommandletClass = InCommandletClass;
		TriggerPropertyChangeEvent(GET_MEMBER_NAME_CHECKED(UWorldPartitionEditorSettings, CommandletClass));
	}
}

int32 UWorldPartitionEditorSettings::GetInstancedFoliageGridSize() const
{
	return InstancedFoliageGridSize;
}

void UWorldPartitionEditorSettings::SetInstancedFoliageGridSize(int32 InInstancedFoliageGridSize)
{
	if (InstancedFoliageGridSize != InInstancedFoliageGridSize)
	{
		InstancedFoliageGridSize = InInstancedFoliageGridSize;
		TriggerPropertyChangeEvent(GET_MEMBER_NAME_CHECKED(UWorldPartitionEditorSettings, InstancedFoliageGridSize));
	}
}

int32 UWorldPartitionEditorSettings::GetMinimapLowQualityWorldUnitsPerPixelThreshold() const
{
	return MinimapLowQualityWorldUnitsPerPixelThreshold;
}

void UWorldPartitionEditorSettings::SetMinimapLowQualityWorldUnitsPerPixelThreshold(int32 InMinimapLowQualityWorldUnitsPerPixelThreshold)
{
	if (MinimapLowQualityWorldUnitsPerPixelThreshold != InMinimapLowQualityWorldUnitsPerPixelThreshold)
	{
		MinimapLowQualityWorldUnitsPerPixelThreshold = InMinimapLowQualityWorldUnitsPerPixelThreshold;
		TriggerPropertyChangeEvent(GET_MEMBER_NAME_CHECKED(UWorldPartitionEditorSettings, MinimapLowQualityWorldUnitsPerPixelThreshold));
	}
}

bool UWorldPartitionEditorSettings::GetEnableLoadingInEditor() const
{
	return bEnableLoadingInEditor;
}

void UWorldPartitionEditorSettings::SetEnableLoadingInEditor(bool bInEnableLoadingInEditor)
{
	if (bEnableLoadingInEditor != bInEnableLoadingInEditor)
	{
		bEnableLoadingInEditor = bInEnableLoadingInEditor;
		TriggerPropertyChangeEvent(GET_MEMBER_NAME_CHECKED(UWorldPartitionEditorSettings, bEnableLoadingInEditor));
	}
}

bool UWorldPartitionEditorSettings::GetEnableStreamingGenerationLogOnPIE() const
{
	return bEnableStreamingGenerationLogOnPIE;
}

void UWorldPartitionEditorSettings::SetEnableStreamingGenerationLogOnPIE(bool bInEnableStreamingGenerationLogOnPIE)
{
	if (bEnableStreamingGenerationLogOnPIE != bInEnableStreamingGenerationLogOnPIE)
	{
		bEnableStreamingGenerationLogOnPIE = bInEnableStreamingGenerationLogOnPIE;
		TriggerPropertyChangeEvent(GET_MEMBER_NAME_CHECKED(UWorldPartitionEditorSettings, bEnableStreamingGenerationLogOnPIE));
	}
}

bool UWorldPartitionEditorSettings::GetShowHLODsInEditor() const
{
	return bShowHLODsInEditor;
}

void UWorldPartitionEditorSettings::SetShowHLODsInEditor(bool bInShowHLODsInEditor)
{
	if (bShowHLODsInEditor != bInShowHLODsInEditor)
	{
		bShowHLODsInEditor = bInShowHLODsInEditor;
		SaveConfig();
		TriggerPropertyChangeEvent(GET_MEMBER_NAME_CHECKED(UWorldPartitionEditorSettings, bShowHLODsInEditor));
	}
}

bool UWorldPartitionEditorSettings::GetShowHLODsOverLoadedRegions() const
{
	return bShowHLODsOverLoadedRegions;
}

void UWorldPartitionEditorSettings::SetShowHLODsOverLoadedRegions(bool bInShowHLODsOverLoadedRegions)
{
	if (bShowHLODsOverLoadedRegions != bInShowHLODsOverLoadedRegions)
	{
		bShowHLODsOverLoadedRegions = bInShowHLODsOverLoadedRegions;
		SaveConfig();
		TriggerPropertyChangeEvent(GET_MEMBER_NAME_CHECKED(UWorldPartitionEditorSettings, bShowHLODsOverLoadedRegions));
	}
}

bool UWorldPartitionEditorSettings::GetEnableAdvancedHLODSettings() const
{
	return bEnableAdvancedHLODSettings;
}

void UWorldPartitionEditorSettings::SetEnableAdvancedHLODSettings(bool bInEnableAdvancedHLODSettings)
{
	if (bEnableAdvancedHLODSettings != bInEnableAdvancedHLODSettings)
	{
		bEnableAdvancedHLODSettings = bInEnableAdvancedHLODSettings;
		TriggerPropertyChangeEvent(GET_MEMBER_NAME_CHECKED(UWorldPartitionEditorSettings, bEnableAdvancedHLODSettings));
	}
}

double UWorldPartitionEditorSettings::GetHLODMinDrawDistance() const
{
	return HLODMinDrawDistance;
}

void UWorldPartitionEditorSettings::SetHLODMinDrawDistance(double InHLODMinDrawDistance)
{
	if (HLODMinDrawDistance != InHLODMinDrawDistance)
	{
		HLODMinDrawDistance = InHLODMinDrawDistance;
		SaveConfig();
		TriggerPropertyChangeEvent(GET_MEMBER_NAME_CHECKED(UWorldPartitionEditorSettings, HLODMinDrawDistance));
	}
}

double UWorldPartitionEditorSettings::GetHLODMaxDrawDistance() const
{
	return HLODMaxDrawDistance;
}

void UWorldPartitionEditorSettings::SetHLODMaxDrawDistance(double InHLODMaxDrawDistance)
{
	if (HLODMaxDrawDistance != InHLODMaxDrawDistance)
	{
		HLODMaxDrawDistance = InHLODMaxDrawDistance;
		SaveConfig();
		TriggerPropertyChangeEvent(GET_MEMBER_NAME_CHECKED(UWorldPartitionEditorSettings, HLODMaxDrawDistance));
	}
}

bool UWorldPartitionEditorSettings::GetDisableBugIt() const
{
	return bDisableBugIt;
}

void UWorldPartitionEditorSettings::SetDisableBugIt(bool bInDisableBugIt)
{
	bDisableBugIt = bInDisableBugIt;
}

bool UWorldPartitionEditorSettings::GetDisablePIE() const
{
	return bDisablePIE;
}

void UWorldPartitionEditorSettings::SetDisablePIE(bool bInDisablePIE)
{
	bDisablePIE = bInDisablePIE;
}

bool UWorldPartitionEditorSettings::GetAdvancedMode() const
{
	return bAdvancedMode;
}

void UWorldPartitionEditorSettings::SetAdvancedMode(bool bInAdvancedMode)
{
	bAdvancedMode = bInAdvancedMode;
}

void UWorldPartitionEditorSettings::TriggerPropertyChangeEvent(FName PropertyName)
{
	FPropertyChangedEvent PCE(GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UWorldPartitionEditorSettings, CommandletClass)), EPropertyChangeType::ValueSet);
	PostEditChangeProperty(PCE);
}

void UWorldPartitionEditorSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UWorldPartitionEditorSettings, bEnableLoadingInEditor))
		{
			if (UWorldPartition* WorldPartition = GWorld ? GWorld->GetWorldPartition() : nullptr)
			{
				WorldPartition->OnEnableLoadingInEditorChanged();
			}
		}

		SettingsChangedDelegate.Broadcast(PropertyChangedEvent.Property->GetFName(), *this);
	}	
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
