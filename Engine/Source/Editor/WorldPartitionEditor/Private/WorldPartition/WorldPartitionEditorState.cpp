// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartition/WorldPartitionEditorState.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterShape.h"
#include "WorldPartition/WorldPartitionEditorLoaderAdapter.h"
#include "WorldPartition/WorldPartitionEditorSettings.h"
#include "Engine/World.h"
#include "LocationVolume.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldPartitionEditorState)

#define LOCTEXT_NAMESPACE "WorldPartitionEditorState"

UWorldPartitionEditorState::UWorldPartitionEditorState(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UWorldPartitionEditorState::GetCategoryText() const
{
	return FText(LOCTEXT("WorldPartitionEditorStateCategoryText", "World Partition"));
}

UEditorState::FOperationResult UWorldPartitionEditorState::CaptureState()
{
	UWorld* CurrentWorld = GetStateWorld();
	if (!ensure(CurrentWorld) || !CurrentWorld->IsPartitionedWorld())
	{
		return FOperationResult(FOperationResult::Skipped, LOCTEXT("CaptureStateSkipped_WorldIsNotPartitioned", "World is not partitioned"));
	}

	if (!GetDefault<UWorldPartitionEditorSettings>()->GetEnableLoadingInEditor())
	{
		return FOperationResult(FOperationResult::Skipped, LOCTEXT("CaptureStateSkipped_LoadingInEditorDisabled", "Loading in editor is disabled"));
	}

	auto GetRegionHash = [](const FBox& LoadedRegion)
	{
		return HashCombineFast(GetTypeHash(LoadedRegion.Min), GetTypeHash(LoadedRegion.Max));
	};

	// Loaded regions
	TSet<uint32> LoadedEditorRegionsHashes;
	for (const FBox& LoadedRegion : CurrentWorld->GetWorldPartition()->GetUserLoadedEditorRegions())
	{
		bool bAlreadyInSet = false;
		LoadedEditorRegionsHashes.FindOrAdd(GetRegionHash(LoadedRegion), &bAlreadyInSet);
		if (!bAlreadyInSet)
		{
			LoadedEditorRegions.Add(LoadedRegion);
		}
	}

	// Loaded location volumes
	for (FActorDescContainerInstanceCollection::TConstIterator<> Iterator(CurrentWorld->GetWorldPartition()); Iterator; ++Iterator)
	{
		if (ALocationVolume* LocationVolume = Cast<ALocationVolume>(Iterator->GetActor()); IsValid(LocationVolume))
		{
			check(LocationVolume->GetClass()->ImplementsInterface(UWorldPartitionActorLoaderInterface::StaticClass()));

			IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = Cast<IWorldPartitionActorLoaderInterface>(LocationVolume)->GetLoaderAdapter();
			check(LoaderAdapter);

			if (LoaderAdapter->IsLoaded() && LoaderAdapter->GetUserCreated())
			{
				LoadedEditorLocationVolumes.Add(LocationVolume->GetFName());
			}
		}
	}

	if (LoadedEditorRegions.IsEmpty() && LoadedEditorLocationVolumes.IsEmpty())
	{
		return FOperationResult(FOperationResult::Skipped, LOCTEXT("CaptureStateSkipped_NoLoadedRegionsOrVolumes", "No regions or location volumes currently loaded"));
	}

	return FOperationResult(FOperationResult::Success, FText::Format(LOCTEXT("CaptureStateSuccess", "{0} regions and {1} location volumes currently loaded"), LoadedEditorRegions.Num(), LoadedEditorLocationVolumes.Num()));
}

UEditorState::FOperationResult UWorldPartitionEditorState::RestoreState() const
{
	UWorld* CurrentWorld = GetStateWorld();
	if (!ensure(CurrentWorld) || !CurrentWorld->IsPartitionedWorld())
	{
		return FOperationResult(FOperationResult::Skipped, LOCTEXT("RestoreStateSkipped_WorldIsNotPartitioned", "World is not partitioned"));
	}

	if (!GetDefault<UWorldPartitionEditorSettings>()->GetEnableLoadingInEditor())
	{
		return FOperationResult(FOperationResult::Skipped, LOCTEXT("RestoreStateSkipped_LoadingInEditorDisabled", "Loading in editor is disabled"));
	}

	auto GetRegionHash = [](const FBox& LoadedRegion)
	{
		// Round region's min/max to avoid precision issues.
		FInt64Vector LoadedRegionMin(FMath::RoundToInt(LoadedRegion.Min.X), FMath::RoundToInt(LoadedRegion.Min.Y), FMath::RoundToInt(LoadedRegion.Min.Z));
		FInt64Vector LoadedRegionMax(FMath::RoundToInt(LoadedRegion.Max.X), FMath::RoundToInt(LoadedRegion.Max.Y), FMath::RoundToInt(LoadedRegion.Max.Z));
		return HashCombineFast(GetTypeHash(LoadedRegionMin), GetTypeHash(LoadedRegionMax));
	};

	// Grab existing loaded regions to avoid creating duplicate regions
	TSet<uint32> LoadedEditorRegionsHashes;
	for (const FBox& LoadedRegion : CurrentWorld->GetWorldPartition()->GetUserLoadedEditorRegions())
	{
		LoadedEditorRegionsHashes.FindOrAdd(GetRegionHash(LoadedRegion));
	}

	// Loaded regions
	int32 NbRegionsLoaded = 0;
	for (const FBox& LoadedRegion : LoadedEditorRegions)
	{
		if (LoadedRegion.IsValid)
		{
			bool bAlreadyInSet = false;
			LoadedEditorRegionsHashes.FindOrAdd(GetRegionHash(LoadedRegion), &bAlreadyInSet);
			if (!bAlreadyInSet)
			{
				UWorldPartitionEditorLoaderAdapter* EditorLoaderAdapter = CurrentWorld->GetWorldPartition()->CreateEditorLoaderAdapter<FLoaderAdapterShape>(CurrentWorld, LoadedRegion, TEXT("Loaded Region"));
				IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = EditorLoaderAdapter->GetLoaderAdapter();
				check(LoaderAdapter);
				LoaderAdapter->SetUserCreated(true);
				LoaderAdapter->Load();
				NbRegionsLoaded++;
			}
		}
	}

	// Location volumes
	int32 NbVolumesLoaded = 0;
	for (const FName& EditorLoadedLocationVolume : LoadedEditorLocationVolumes)
	{
		if (ALocationVolume* LocationVolume = FindObject<ALocationVolume>(CurrentWorld->PersistentLevel, *EditorLoadedLocationVolume.ToString()))
		{
			LocationVolume->Load();
			NbVolumesLoaded++;
		}
	}
	
	return FOperationResult(FOperationResult::Success, FText::Format(LOCTEXT("RestoreStateSuccess", "Loaded {0} regions and {1} location volumes"), NbRegionsLoaded, NbVolumesLoaded));
}

#undef LOCTEXT_NAMESPACE
