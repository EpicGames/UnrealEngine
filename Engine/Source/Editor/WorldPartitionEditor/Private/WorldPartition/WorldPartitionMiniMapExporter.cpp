// Copyright Epic Games, Inc. All Rights Reserved.

#include "Editor.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "HAL/IConsoleManager.h"
#include "ImageUtils.h"
#include "Misc/DateTime.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/JsonWriter.h"
#include "WorldPartition/WorldPartitionLog.h"
#include "WorldPartition/WorldPartitionMiniMap.h"
#include "WorldPartition/WorldPartitionMiniMapHelper.h"

static FAutoConsoleCommand ExportMinimapForInsightsCmd(
	TEXT("wp.Editor.ExportMinimapForInsights"),
	TEXT("Export minimap PNG + sidecar JSON for Unreal Insights Spatial Profiler.\n")
	TEXT("Usage: wp.Editor.ExportMinimapForInsights [OutputPath]  (no extension; writes <OutputPath>.png + <OutputPath>.png.json. Default: <ProjectSaved>/Profiling/WorldPartitionMinimaps/<MapName>_<timestamp>)"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& InArgs)
	{
		UWorld* EditorWorld = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr;
		if (!EditorWorld)
		{
			UE_LOGF(LogWorldPartition, Error, "No editor world is currently loaded.");
			return;
		}
		if (EditorWorld->IsGameWorld())
		{
			UE_LOGF(LogWorldPartition, Error, "Cannot export minimap while a game world is active (e.g. PIE).");
			return;
		}

		AWorldPartitionMiniMap* WorldMiniMap = FWorldPartitionMiniMapHelper::GetWorldPartitionMiniMap(EditorWorld, /*bCreateNewMiniMap=*/false);
		if (!WorldMiniMap)
		{
			UE_LOGF(LogWorldPartition, Error, "No World Partition minimap actor found. Run WorldPartitionMiniMapBuilder to generate one.");
			return;
		}

		const FString OutputBasePath = (InArgs.Num() >= 1 && !InArgs[0].IsEmpty()) ? InArgs[0] : FPaths::ProjectSavedDir() / TEXT("Profiling") / TEXT("WorldPartitionMinimaps") / FString::Printf(TEXT("%s_%s"), *FPaths::MakeValidFileName(EditorWorld->GetMapName(), TEXT('_')), *FDateTime::Now().ToString());

		UTexture2D* MiniMapTexture = WorldMiniMap->MiniMapTexture;
		if (!MiniMapTexture)
		{
			UE_LOGF(LogWorldPartition, Error, "Minimap actor exists but has no texture.");
			return;
		}

		const FBox& Bounds = WorldMiniMap->MiniMapWorldBounds;
		if (!Bounds.IsValid || Bounds.GetSize().IsNearlyZero())
		{
			UE_LOGF(LogWorldPartition, Error, "Minimap world bounds are invalid or degenerate.");
			return;
		}

		const int32 NumBlocks = MiniMapTexture->Source.GetNumBlocks();
		if (NumBlocks <= 0)
		{
			UE_LOGF(LogWorldPartition, Error, "Minimap texture source has no blocks.");
			return;
		}

		const FIntPoint LogicalSize = MiniMapTexture->Source.GetLogicalSize();
		const FIntPoint BlockGrid = MiniMapTexture->Source.GetSizeInBlocks();
		if (LogicalSize.X <= 0 || LogicalSize.Y <= 0 || BlockGrid.X <= 0 || BlockGrid.Y <= 0)
		{
			UE_LOGF(LogWorldPartition, Error, "Minimap texture has invalid logical size (%dx%d) or block grid (%dx%d).", LogicalSize.X, LogicalSize.Y, BlockGrid.X, BlockGrid.Y);
			return;
		}

		const int32 BlockSizeX = LogicalSize.X / BlockGrid.X;
		const int32 BlockSizeY = LogicalSize.Y / BlockGrid.Y;

		FImage SourceImage;
		SourceImage.Init(LogicalSize.X, LogicalSize.Y, ERawImageFormat::BGRA8, EGammaSpace::sRGB);

		const int64 DestinationStride = SourceImage.GetStrideBytes();

		for (int32 BlockIndex = 0; BlockIndex < NumBlocks; ++BlockIndex)
		{
			FTextureSourceBlock Block;
			MiniMapTexture->Source.GetBlock(BlockIndex, Block);

			if (Block.SizeX != BlockSizeX || Block.SizeY != BlockSizeY)
			{
				UE_LOGF(LogWorldPartition, Error, "Minimap UDIM block %d has non-uniform size (%dx%d, expected %dx%d).", BlockIndex, Block.SizeX, Block.SizeY, BlockSizeX, BlockSizeY);
				return;
			}

			FImage BlockImage;
			if (!MiniMapTexture->Source.GetMipImage(BlockImage, BlockIndex, /*LayerIndex=*/0, /*MipIndex=*/0))
			{
				UE_LOGF(LogWorldPartition, Error, "Failed to read source block %d.", BlockIndex);
				return;
			}

			FImage BlockImageBGRA8;
			BlockImage.CopyTo(BlockImageBGRA8, ERawImageFormat::BGRA8, EGammaSpace::sRGB);

			// Flip Y because UDIM block (0,0) is bottom-left while the destination image is top-left origin.
			const int32 DestinationX = Block.BlockX * BlockSizeX;
			const int32 DestinationY = (BlockGrid.Y - 1 - Block.BlockY) * BlockSizeY;
			const int64 SourceStride = BlockImageBGRA8.GetStrideBytes();
			const uint8* SourceBlockPixels = static_cast<const uint8*>(BlockImageBGRA8.GetPixelPointer(0, 0));
			uint8* DestinationPixels = static_cast<uint8*>(SourceImage.GetPixelPointer(DestinationX, DestinationY));
			for (int32 Row = 0; Row < BlockSizeY; ++Row)
			{
				FMemory::Memcpy(DestinationPixels + Row * DestinationStride, SourceBlockPixels + Row * SourceStride, SourceStride);
			}
		}

		// SCS_BaseColor capture writes alpha=0 to Source; the AdjustMinAlpha=1 remap is GPU-only so Source readback would produce a transparent PNG.
		FImageCore::SetAlphaOpaque(SourceImage);

		const FString PngPath = OutputBasePath + TEXT(".png");
		if (!FImageUtils::SaveImageByExtension(*PngPath, SourceImage))
		{
			UE_LOGF(LogWorldPartition, Error, "Failed to write PNG to: %ls", *PngPath);
			return;
		}

		// Write sidecar JSON with world bounds (XY only).
		FString JsonContent;
		{
			TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&JsonContent);
			Writer->WriteObjectStart();
			Writer->WriteArrayStart(TEXT("worldBoundsMin"));
			Writer->WriteValue(Bounds.Min.X);
			Writer->WriteValue(Bounds.Min.Y);
			Writer->WriteArrayEnd();
			Writer->WriteArrayStart(TEXT("worldBoundsMax"));
			Writer->WriteValue(Bounds.Max.X);
			Writer->WriteValue(Bounds.Max.Y);
			Writer->WriteArrayEnd();
			Writer->WriteObjectEnd();
			Writer->Close();
		}

		const FString JsonPath = PngPath + TEXT(".json");
		if (!FFileHelper::SaveStringToFile(JsonContent, *JsonPath, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
		{
			UE_LOGF(LogWorldPartition, Error, "Failed to write JSON to: %ls", *JsonPath);
			return;
		}

		UE_LOGF(LogWorldPartition, Display, "Minimap exported:");
		UE_LOGF(LogWorldPartition, Display, "  PNG:  %ls (%" INT64_FMT "x%" INT64_FMT ")", *PngPath, SourceImage.GetWidth(), SourceImage.GetHeight());
		UE_LOGF(LogWorldPartition, Display, "  JSON: %ls", *JsonPath);
		UE_LOGF(LogWorldPartition, Display, "  Bounds: Min(%.1f, %.1f) Max(%.1f, %.1f)", Bounds.Min.X, Bounds.Min.Y, Bounds.Max.X, Bounds.Max.Y);
	})
);
