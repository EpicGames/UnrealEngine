// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldPartitionEditorModule.h"

#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHLODsBuilder.h"
#include "WorldPartition/WorldPartitionMiniMapBuilder.h"
#include "WorldPartition/WorldPartitionLandscapeSplineMeshesBuilder.h"
#include "WorldPartition/WorldPartitionActorLoaderInterface.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterShape.h"
#include "WorldPartition/SWorldPartitionEditor.h"
#include "WorldPartition/SWorldPartitionEditorGridSpatialHash.h"
#include "WorldPartition/Customizations/ExternalDataLayerUIDStructCustomization.h"
#include "WorldPartition/Customizations/WorldPartitionDetailsCustomization.h"
#include "WorldPartition/Customizations/WorldPartitionHLODDetailsCustomization.h"
#include "WorldPartition/Customizations/WorldPartitionHLODLayerDetailsCustomization.h"
#include "WorldPartition/Customizations/WorldPartitionRuntimeSpatialHashDetailsCustomization.h"
#include "WorldPartition/Customizations/WorldPartitionRuntimeHashSetDetailsCustomization.h"
#include "WorldPartition/Customizations/WorldDataLayersActorDetails.h"
#include "WorldPartition/Customizations/WorldPartitionEditorPerProjectUserSettingsDetails.h"
#include "WorldPartition/SWorldPartitionConvertDialog.h"
#include "WorldPartition/WorldPartitionConvertOptions.h"
#include "WorldPartition/WorldPartitionEditorSettings.h"
#include "WorldPartition/HLOD/HLODActor.h"
#include "WorldPartition/HLOD/HLODCreationFilter.h"
#include "WorldPartition/HLOD/SWorldPartitionBuildHLODsDialog.h"
#include "WorldPartition/HLOD/HLODEditorSubsystem.h"
#include "WorldPartition/HLOD/SHLODCompareWindow.h"
#include "WorldPartition/WorldPartitionClassDescRegistry.h"
#include "WorldPartition/DataLayer/ExternalDataLayerUID.h"
#include "WorldPartition/DataLayer/ExternalDataLayerHelper.h"

#include "LevelEditor.h"
#include "LevelEditorViewport.h"
#include "SLevelViewport.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "Editor/AssetReferenceFilter.h"
#include "Engine/Level.h"

#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/StringBuilder.h"
#include "Misc/ScopedSlowTask.h"
#include "Internationalization/Regex.h"

#include "Interfaces/IMainFrameModule.h"
#include "Widgets/SWindow.h"
#include "Commandlets/WorldPartitionConvertCommandlet.h"
#include "FileHelpers.h"
#include "ToolMenus.h"
#include "IContentBrowserSingleton.h"
#include "IDirectoryWatcher.h"
#include "DirectoryWatcherModule.h"
#include "ContentBrowserModule.h"
#include "EditorDirectories.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "PropertyEditorModule.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Widgets/Docking/SDockTab.h"
#include "Filters/CustomClassFilterData.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "SourceControlOperations.h"
#include "SceneOutlinerPublicTypes.h"
#include "SceneOutlinerModule.h"

#include "Styling/AppStyle.h"
#include "WorldPartition/ContentBundle/SContentBundleBrowser.h"
#include "Selection.h"
#include "WorldPartition/ContentBundle/ContentBundleEditorSubsystem.h"
#include "UObject/UObjectBase.h"
#include "WorldPartition/ContentBundle/ContentBundleEditor.h"

#include "EditorState/EditorStateSubsystem.h"
#include "WorldPartition/WorldPartitionEditorState.h"

IMPLEMENT_MODULE( FWorldPartitionEditorModule, WorldPartitionEditor );

#define LOCTEXT_NAMESPACE "WorldPartition"

const FName WorldPartitionEditorTabId("WorldBrowserPartitionEditor");
const FName ContentBundleBrowserTabId("ContentBundleBrowser");

DEFINE_LOG_CATEGORY_STATIC(LogWorldPartitionEditor, All, All);

static void OnSelectedWorldPartitionVolumesToggleLoading(TArray<TWeakObjectPtr<AActor>> Volumes, bool bLoad)
{
	for (TWeakObjectPtr<AActor> Actor: Volumes)
	{
		if (Actor->Implements<UWorldPartitionActorLoaderInterface>())
		{
			if (IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = Cast<IWorldPartitionActorLoaderInterface>(Actor)->GetLoaderAdapter())
			{
				if (bLoad)
				{
					LoaderAdapter->Load();
				}
				else
				{
					LoaderAdapter->Unload();
				}
			}
		}
	}
}

static bool CanLoadUnloadSelectedVolumes(TArray<TWeakObjectPtr<AActor>> Volumes, bool bLoad)
{
	for (TWeakObjectPtr<AActor> Actor : Volumes)
	{
		if (Actor->Implements<UWorldPartitionActorLoaderInterface>())
		{
			if (IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter = Cast<IWorldPartitionActorLoaderInterface>(Actor)->GetLoaderAdapter())
			{
				if (bLoad != LoaderAdapter->IsLoaded())
				{
					return true;
				}
			}
		}
	}

	return false;
}

static void CreateLevelViewportContextMenuEntries(FMenuBuilder& MenuBuilder, TArray<TWeakObjectPtr<AActor>> Volumes, FBox SelectionBox, TSharedPtr<FHLODCreationVolumeFilter> VolumeFilter, TArray<TWeakObjectPtr<AWorldPartitionHLOD>> CompareHLODActors)
{
	MenuBuilder.BeginSection("WorldPartition", LOCTEXT("WorldPartition", "World Partition"));
	
	if (!Volumes.IsEmpty())
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("WorldPartitionLoad", "Load selected volumes"),
			LOCTEXT("WorldPartitionLoad_Tooltip", "Load selected volumes"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(OnSelectedWorldPartitionVolumesToggleLoading, Volumes, true),
				FCanExecuteAction::CreateLambda([Volumes]
				{
					return CanLoadUnloadSelectedVolumes(Volumes, true);
				})
			),
			NAME_None,
			EUserInterfaceActionType::Button);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("WorldPartitionUnload", "Unload selected volumes"),
			LOCTEXT("WorldPartitionUnload_Tooltip", "Load selected volumes"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateStatic(OnSelectedWorldPartitionVolumesToggleLoading, Volumes, false),
				FCanExecuteAction::CreateLambda([Volumes]
				{
					return CanLoadUnloadSelectedVolumes(Volumes, false);
				})
			),
			NAME_None,
			EUserInterfaceActionType::Button);
	}

	// Load Region From Selection
	if (GCurrentLevelEditingViewportClient && SelectionBox.GetSize().Size2D() > 0)
	{
		TWeakObjectPtr<UWorld> World = GCurrentLevelEditingViewportClient->GetWorld();
		TWeakObjectPtr<UWorldPartition> WorldPartition = World->GetWorldPartition();

		FUIAction LoadRegion(
			FExecuteAction::CreateLambda([World, WorldPartition, SelectionBox]()
			{
				if (World.IsValid() && WorldPartition.IsValid())
				{
					UWorldPartitionEditorLoaderAdapter* EditorLoaderAdapter = WorldPartition.Get()->CreateEditorLoaderAdapter<FLoaderAdapterShape>(World.Get(), SelectionBox, TEXT("Loaded Region"));
					EditorLoaderAdapter->GetLoaderAdapter()->SetUserCreated(true);
					EditorLoaderAdapter->GetLoaderAdapter()->Load();
				}
			}),
			FCanExecuteAction::CreateLambda([World, WorldPartition]()
			{
				return (World.IsValid() && WorldPartition.IsValid());
			})
		);



		MenuBuilder.AddMenuEntry(LOCTEXT("LoadRegionFromSelection", "Load Region From Selection"), LOCTEXT("LoadRegionFromSelection_Tooltip", "Load region from selected actor(s) bounds"), FSlateIcon(), LoadRegion);
	}

	if (GCurrentLevelEditingViewportClient && VolumeFilter->IsActive())
	{
		TWeakObjectPtr<UWorld> World = GCurrentLevelEditingViewportClient->GetWorld();

		FUIAction BuildHLODsFromSelection(
			FExecuteAction::CreateLambda([World, VolumeFilter]()
			{
				if (World.IsValid())
				{
					UWorldPartitionHLODsBuilder::BuildWithFilters(World.Get(), { VolumeFilter });
				}
			}),
			FCanExecuteAction::CreateLambda([World]()
			{
				if (World.IsValid())
				{
					TWeakObjectPtr<UWorldPartition> WorldPartition = World->GetWorldPartition();
					return WorldPartition.IsValid() && WorldPartition->IsStreamingEnabled();
				}
				return false;
			})
		);

		MenuBuilder.AddMenuEntry(LOCTEXT("BuildHLODsFromSelection", "Build HLODs From Selection"), LOCTEXT("BuildHLODSFromSelection_Tooltip", "Build HLODs from selected actor(s) bounds"), FSlateIcon(), BuildHLODsFromSelection);
	}

	// Compare HLOD against source actors (only when all selected actors are HLOD actors)
	if (!CompareHLODActors.IsEmpty())
	{
		FUIAction CompareHLODAction(
			FExecuteAction::CreateLambda([CompareHLODActors]()
			{
				FText WindowTitle = (CompareHLODActors.Num() == 1 && CompareHLODActors[0].IsValid())
					? FText::Format(LOCTEXT("CompareWindowTitleSingle", "Compare HLOD: {0}"), FText::FromString(CompareHLODActors[0]->GetActorLabel()))
					: FText::Format(LOCTEXT("CompareWindowTitleMulti", "Compare HLOD ({0} HLOD actors)"), FText::AsNumber(CompareHLODActors.Num()));

				TSharedRef<SWindow> Window = SNew(SWindow)
					.Title(WindowTitle)
					.ClientSize(FVector2D(1280, 720))
					.SizingRule(ESizingRule::UserSized)
					.SupportsMinimize(true)
					.SupportsMaximize(true);

				Window->SetContent(
					SNew(SHLODCompareWindow)
					.HLODActors(CompareHLODActors)
				);

				FSlateApplication::Get().AddWindow(Window);
			})
		);

		MenuBuilder.AddMenuEntry(LOCTEXT("CompareHLODAgainstSource", "Compare HLOD against source"), LOCTEXT("CompareHLODAgainstSource_Tooltip", "Open a wipe comparison of the selected HLODs against their source actors"), FSlateIcon(), CompareHLODAction);
	}

	MenuBuilder.EndSection();
}

static TSharedRef<FExtender> OnExtendLevelEditorMenu(const TSharedRef<FUICommandList> CommandList, TArray<AActor*> SelectedActors)
{
	TSharedRef<FExtender> Extender(new FExtender());

	TArray<TWeakObjectPtr<AActor>> Volumes;
	FBoxSphereBounds::Builder BoundsBuilder;
	TArray<FBox> FilterVolumes;

	for (AActor* Actor : SelectedActors)
	{
		if (Actor->Implements<UWorldPartitionActorLoaderInterface>())
		{
			Volumes.Add(Actor);
		}

		FBoxSphereBounds ActorBounds;
		Actor->GetActorBounds(false, ActorBounds.Origin, ActorBounds.BoxExtent);
		BoundsBuilder += ActorBounds;

		FBox RuntimeBounds;
		FBox EditorBounds;
		Actor->GetStreamingBounds(RuntimeBounds, EditorBounds);
		FilterVolumes.Add(RuntimeBounds);
	}

	TSharedPtr<FHLODCreationVolumeFilter> VolumeFilter = MakeShared<FHLODCreationVolumeFilter>(MoveTemp(FilterVolumes));

	// Collect HLOD actors for the compare entry (only if ALL selected actors are HLOD actors)
	TArray<TWeakObjectPtr<AWorldPartitionHLOD>> CompareHLODActors;
	bool bAllAreHLOD = !SelectedActors.IsEmpty();
	for (AActor* Actor : SelectedActors)
	{
		if (AWorldPartitionHLOD* HLODActor = Cast<AWorldPartitionHLOD>(Actor))
		{
			CompareHLODActors.Add(HLODActor);
		}
		else
		{
			bAllAreHLOD = false;
		}
	}
	if (!bAllAreHLOD)
	{
		CompareHLODActors.Empty();
	}

	if (!Volumes.IsEmpty() || BoundsBuilder.IsValid() || VolumeFilter->IsActive() || !CompareHLODActors.IsEmpty())
	{
		Extender->AddMenuExtension(
			"ActorTypeTools",
			EExtensionHook::After,
			nullptr,
			FMenuExtensionDelegate::CreateStatic(&CreateLevelViewportContextMenuEntries, Volumes, FBoxSphereBounds(BoundsBuilder).GetBox(), VolumeFilter, CompareHLODActors));
	}

	return Extender;
}

void FWorldPartitionEditorModule::StartupModule()
{
	SWorldPartitionEditorGrid::RegisterPartitionEditorGridCreateInstanceFunc(NAME_None, &SWorldPartitionEditorGrid::CreateInstance);
	SWorldPartitionEditorGrid::RegisterPartitionEditorGridCreateInstanceFunc(TEXT("SpatialHash"), &SWorldPartitionEditorGridSpatialHash::CreateInstance);

	FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditor.RegisterCustomClassLayout("WorldPartition", FOnGetDetailCustomizationInstance::CreateStatic(&FWorldPartitionDetails::MakeInstance));
	PropertyEditor.RegisterCustomClassLayout("WorldPartitionRuntimeSpatialHash", FOnGetDetailCustomizationInstance::CreateStatic(&FWorldPartitionRuntimeSpatialHashDetails::MakeInstance));
	PropertyEditor.RegisterCustomPropertyTypeLayout("RuntimePartitionHLODSetup", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FRuntimePartitionHLODSetupDetails::MakeInstance));
	PropertyEditor.RegisterCustomClassLayout("WorldPartitionHLOD", FOnGetDetailCustomizationInstance::CreateStatic(&FWorldPartitionHLODDetailsCustomization::MakeInstance));
	PropertyEditor.RegisterCustomClassLayout("HLODLayer", FOnGetDetailCustomizationInstance::CreateStatic(&FWorldPartitionHLODLayerDetailsCustomization::MakeInstance));
	PropertyEditor.RegisterCustomClassLayout("WorldDataLayers", FOnGetDetailCustomizationInstance::CreateStatic(&FWorldDataLayersActorDetails::MakeInstance));
	PropertyEditor.RegisterCustomClassLayout("WorldPartitionEditorPerProjectUserSettings", FOnGetDetailCustomizationInstance::CreateStatic(&FWorldPartitionEditorPerProjectUserSettingsCustomization::MakeInstance));
	PropertyEditor.RegisterCustomPropertyTypeLayout("ExternalDataLayerUID", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FExternalDataLayerUIDStructCustomization::MakeInstance));

	FWorldPartitionClassDescRegistry().Get().Initialize();

	EditorInitializedHandle = FEditorDelegates::OnEditorInitialized.AddLambda([this](double TimeToInitializeEditor)
	{
		UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FWorldPartitionEditorModule::RegisterMenus));

		// Register the Scene Outliner "World" filter category
		if (FModuleManager::Get().IsModuleLoaded("LevelEditor"))
		{
			FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
			TSharedPtr<FFilterCategory> CommonFilterCategory = LevelEditorModule.GetOutlinerFilterCategory(FLevelEditorOutlinerBuiltInCategories::Common());
			TSharedPtr<FFilterCategory> WorldFilterCategory = MakeShared<FFilterCategory>(LOCTEXT("WorldFilterCategory", "World"), FText::GetEmpty());

			TArray<UClass*> WorldActorClasses =
			{
				AWorldPartitionHLOD::StaticClass()
			};

			for (UClass* Class : WorldActorClasses)
			{
				TSharedRef<FCustomClassFilterData> ClassFilterData = MakeShared<FCustomClassFilterData>(AWorldPartitionHLOD::StaticClass(), WorldFilterCategory, FLinearColor::White);

				if (CommonFilterCategory.IsValid())
				{
					ClassFilterData->AddCategory(CommonFilterCategory);
				}

				LevelEditorModule.AddCustomClassFilterToOutliner(ClassFilterData);
			}
		}

		UEditorStateSubsystem::RegisterEditorStateType<UWorldPartitionEditorState>();

		EditorCloseHandle = GEditor->OnEditorClose().AddLambda([this]()
		{
			UEditorStateSubsystem::UnregisterEditorStateType<UWorldPartitionEditorState>();
			GEditor->OnEditorClose().Remove(EditorCloseHandle);
		});
	});	

	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
	SceneOutlinerModule.OnCreateActorBrowserColumns().AddRaw(this, &FWorldPartitionEditorModule::OnCreateActorBrowserColumns);

	IAssetReferenceFilter::OnIsCrossPluginReferenceAllowed().BindRaw(this, &FWorldPartitionEditorModule::OnIsCrossPluginReferenceAllowed);

	bRequireExplicitHLODLayerPartitionAssignation = true;
}

void FWorldPartitionEditorModule::ShutdownModule()
{
	FWorldPartitionClassDescRegistry().Get().Uninitialize();
	FWorldPartitionClassDescRegistry().Get().TearDown();

	if (FSceneOutlinerModule* SceneOutlinerModule = FModuleManager::GetModulePtr<FSceneOutlinerModule>("SceneOutliner"))
	{
		SceneOutlinerModule->OnCreateActorBrowserColumns().RemoveAll(this);
	}

	IAssetReferenceFilter::OnIsCrossPluginReferenceAllowed().Unbind();

	if (!IsRunningGame())
	{
		if (FLevelEditorModule* LevelEditorModule = FModuleManager::Get().GetModulePtr<FLevelEditorModule>("LevelEditor"))
		{
			LevelEditorModule->GetAllLevelViewportContextMenuExtenders().RemoveAll([this](const FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors& In) { return In.GetHandle() == LevelEditorExtenderDelegateHandle; });

			LevelEditorModule->OnRegisterTabs().RemoveAll(this);
			LevelEditorModule->OnRegisterLayoutExtensions().RemoveAll(this);

			if (LevelEditorModule->GetLevelEditorTabManager())
			{
				LevelEditorModule->GetLevelEditorTabManager()->UnregisterTabSpawner(WorldPartitionEditorTabId);
			}

		}

		UToolMenus::UnRegisterStartupCallback(this);
		UToolMenus::UnregisterOwner(this);
	}

	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyEditor = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditor.UnregisterCustomClassLayout("WorldPartition");
	}

	FEditorDelegates::OnEditorInitialized.Remove(EditorInitializedHandle);
}

void FWorldPartitionEditorModule::OnCreateActorBrowserColumns(FSceneOutlinerInitializationOptions& InitOptions, UWorld* World)
{
	// We don't want the pinned column in non wp levels
	if (World && World->IsPartitionedWorld())
	{
		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Pinned(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 5, FCreateSceneOutlinerColumn(), true, TOptional<float>(), FSceneOutlinerBuiltInColumnTypes::Pinned_Localized()));
	}
}

bool FWorldPartitionEditorModule::OnIsCrossPluginReferenceAllowed(const FAssetData& ReferencingAssetData, const FAssetData& ReferencedAssetData)
{
	// Allow External Data Layer Actor (ReferencingAssetData) from a plugin X to reference its world (ReferencedAssetData) from a plugin Y
	const UClass* ReferencedAssetDataClass = ReferencedAssetData.GetClass();
	if (ReferencedAssetDataClass && ReferencedAssetDataClass->IsChildOf<UWorld>())
	{
		const FString ReferencingAssetPath = ReferencingAssetData.PackagePath.ToString();
		FExternalDataLayerUID ReferencingExternalDataLayerUID;
		if (FExternalDataLayerHelper::IsExternalDataLayerPath(ReferencingAssetPath, &ReferencingExternalDataLayerUID); ReferencingExternalDataLayerUID.IsValid())
		{
			// Use referencing asset's optional outer path name (if any) to build its package name and compare it with the referenced package name
			const FString ReferencingOptionalOuterPackageName = FSoftObjectPath(ReferencingAssetData.GetOptionalOuterPathName().ToString()).GetLongPackageName();
			return ReferencingOptionalOuterPackageName == ReferencedAssetData.PackageName.ToString();
		}
	}

	return false;
}

void FWorldPartitionEditorModule::RegisterMenus()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	TArray<FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors>& MenuExtenderDelegates = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();

	LevelEditorModule.OnRegisterTabs().AddRaw(this, &FWorldPartitionEditorModule::RegisterWorldPartitionTabs);
	LevelEditorModule.OnRegisterLayoutExtensions().AddRaw(this, &FWorldPartitionEditorModule::RegisterWorldPartitionLayout);

	MenuExtenderDelegates.Add(FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors::CreateStatic(&OnExtendLevelEditorMenu));
	LevelEditorExtenderDelegateHandle = MenuExtenderDelegates.Last().GetHandle();

	FToolMenuOwnerScoped OwnerScoped(this);
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Tools");	
	FToolMenuSection& Section = Menu->AddSection("WorldPartition", LOCTEXT("WorldPartition", "World Partition"));
	Section.AddEntry(FToolMenuEntry::InitMenuEntry(
		"WorldPartition",
		LOCTEXT("WorldPartitionConvertTitle", "Convert Level..."),
		LOCTEXT("WorldPartitionConvertTooltip", "Converts a Level to World Partition."),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "DeveloperTools.MenuIcon"),
		FUIAction(FExecuteAction::CreateRaw(this, &FWorldPartitionEditorModule::OnConvertMap))
	));
}

TSharedRef<SWidget> FWorldPartitionEditorModule::CreateWorldPartitionEditor()
{
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
	return SNew(SWorldPartitionEditor).InWorld(EditorWorld);
}

TSharedRef<SWidget> FWorldPartitionEditorModule::CreateContentBundleBrowser()
{
	check(ContentBundleBrowser == nullptr);
	TSharedRef<SContentBundleBrowser> NewDataLayerBrowser = SNew(SContentBundleBrowser);
	ContentBundleBrowser = NewDataLayerBrowser;
	return NewDataLayerBrowser;
}

bool FWorldPartitionEditorModule::IsEditingContentBundle() const
{
	UContentBundleEditorSubsystem* ContentBundleEditorSubsystem = UContentBundleEditorSubsystem::Get();
	return ContentBundleEditorSubsystem && ContentBundleEditorSubsystem->IsEditingContentBundle();
}

bool FWorldPartitionEditorModule::IsEditingContentBundle(const FGuid& ContentBundleGuid) const
{
	UContentBundleEditorSubsystem* ContentBundleEditorSubsystem = UContentBundleEditorSubsystem::Get();
	return ContentBundleEditorSubsystem && ContentBundleEditorSubsystem->IsEditingContentBundle(ContentBundleGuid);
}

bool FWorldPartitionEditorModule::GetActiveLevelViewportCameraInfo(FVector& CameraLocation, FRotator& CameraRotation)
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
	if (LevelEditor.IsValid())
	{
		TSharedPtr<SLevelViewport> LevelViewport = LevelEditor->GetActiveViewportInterface();
		if (LevelViewport.IsValid())
		{
			const FLevelEditorViewportClient& LevelViewportClient = LevelViewport->GetLevelViewportClient();
			CameraLocation = LevelViewportClient.GetViewLocation();
			CameraRotation = LevelViewportClient.GetViewRotation();			
			return true;
		}
	}

	return false;
}

int32 FWorldPartitionEditorModule::GetPlacementGridSize() const
{
	// Currently shares setting with Foliage. Can be changed when exposed.
	return GetDefault<UWorldPartitionEditorSettings>()->GetInstancedFoliageGridSize();
}

int32 FWorldPartitionEditorModule::GetInstancedFoliageGridSize() const
{
	return GetDefault<UWorldPartitionEditorSettings>()->GetInstancedFoliageGridSize();
}

int32 FWorldPartitionEditorModule::GetMinimapLowQualityWorldUnitsPerPixelThreshold() const
{
	return GetDefault<UWorldPartitionEditorSettings>()->GetMinimapLowQualityWorldUnitsPerPixelThreshold();
}

bool FWorldPartitionEditorModule::GetEnableLoadingInEditor() const
{
	return GetDefault<UWorldPartitionEditorSettings>()->GetEnableLoadingInEditor();
}

void FWorldPartitionEditorModule::SetEnableLoadingInEditor(bool bInEnableLoadingInEditor)
{
	GetMutableDefault<UWorldPartitionEditorSettings>()->SetEnableLoadingInEditor(bInEnableLoadingInEditor);
}

bool FWorldPartitionEditorModule::GetEnableStreamingGenerationLogOnPIE() const
{
	return GetDefault<UWorldPartitionEditorSettings>()->GetEnableStreamingGenerationLogOnPIE();
}

void FWorldPartitionEditorModule::SetEnableStreamingGenerationLogOnPIE(bool bEnableStreamingGenerationLogOnPIE)
{
	GetMutableDefault<UWorldPartitionEditorSettings>()->SetEnableStreamingGenerationLogOnPIE(bEnableStreamingGenerationLogOnPIE);
}

bool FWorldPartitionEditorModule::GetDisablePIE() const
{
	return GetDefault<UWorldPartitionEditorSettings>()->GetDisablePIE();
}

void FWorldPartitionEditorModule::SetDisablePIE(bool bInDisablePIE)
{
	GetMutableDefault<UWorldPartitionEditorSettings>()->SetDisablePIE(bInDisablePIE);
}

bool FWorldPartitionEditorModule::GetDisableBugIt() const
{
	return GetDefault<UWorldPartitionEditorSettings>()->GetDisableBugIt();
}

void FWorldPartitionEditorModule::SetDisableBugIt(bool bInDisableBugIt)
{
	GetMutableDefault<UWorldPartitionEditorSettings>()->SetDisableBugIt(bInDisableBugIt);
}

bool FWorldPartitionEditorModule::GetAdvancedMode() const
{
	return GetDefault<UWorldPartitionEditorSettings>()->GetAdvancedMode();
}

void FWorldPartitionEditorModule::SetAdvancedMode(bool bInAdvancedMode)
{
	GetMutableDefault<UWorldPartitionEditorSettings>()->SetAdvancedMode(bInAdvancedMode);
}

bool FWorldPartitionEditorModule::GetRequireExplicitHLODLayerPartitionAssignation() const
{
	return bRequireExplicitHLODLayerPartitionAssignation;
}

void FWorldPartitionEditorModule::SetRequireExplicitHLODLayerPartitionAssignation(bool bInRequireExplicitHLODLayerPartitionAssignation)
{
	bRequireExplicitHLODLayerPartitionAssignation = bInRequireExplicitHLODLayerPartitionAssignation;
}

bool FWorldPartitionEditorModule::GetShowHLODsInEditor() const
{
	return GetDefault<UWorldPartitionEditorSettings>()->GetShowHLODsInEditor();
}

void FWorldPartitionEditorModule::SetShowHLODsInEditor(bool bInShowHLODsInEditor)
{
	GetMutableDefault<UWorldPartitionEditorSettings>()->SetShowHLODsInEditor(bInShowHLODsInEditor);
}

bool FWorldPartitionEditorModule::GetShowHLODsOverLoadedRegions() const
{
	return GetDefault<UWorldPartitionEditorSettings>()->GetShowHLODsOverLoadedRegions();
}

void FWorldPartitionEditorModule::SetShowHLODsOverLoadedRegions(bool bInShowHLODsOverLoadedRegions)
{
	GetMutableDefault<UWorldPartitionEditorSettings>()->SetShowHLODsOverLoadedRegions(bInShowHLODsOverLoadedRegions);
}

double FWorldPartitionEditorModule::GetHLODInEditorMinDrawDistance() const
{
	return GetDefault<UWorldPartitionEditorSettings>()->GetHLODMinDrawDistance();
}

void FWorldPartitionEditorModule::SetHLODInEditorMinDrawDistance(double InMinDrawDistance)
{
	GetMutableDefault<UWorldPartitionEditorSettings>()->SetHLODMinDrawDistance(InMinDrawDistance);
}

double FWorldPartitionEditorModule::GetHLODInEditorMaxDrawDistance() const
{
	return GetDefault<UWorldPartitionEditorSettings>()->GetHLODMaxDrawDistance();
}

void FWorldPartitionEditorModule::SetHLODInEditorMaxDrawDistance(double InMaxDrawDistance)
{
	GetMutableDefault<UWorldPartitionEditorSettings>()->SetHLODMaxDrawDistance(InMaxDrawDistance);
}

bool FWorldPartitionEditorModule::IsHLODInEditorAllowed(UWorld* InWorld, FText* OutDisallowedReason) const
{
	auto SetDissallowedReason = [OutDisallowedReason](const FText& DisallowedReason)
	{
		if (OutDisallowedReason)
		{
			*OutDisallowedReason = DisallowedReason;
		}
	};

	if (!InWorld)
	{
		SetDissallowedReason(LOCTEXT("HLODInEditor_InvalidWorld", "Invalid world"));
		return false;
	}

	if (!InWorld->IsPartitionedWorld())
	{
		SetDissallowedReason(LOCTEXT("HLODInEditor_NoWorldPartition", "World is non partitioned"));
		return false;
	}

	if (!InWorld->GetWorldPartition()->IsStreamingEnabledInEditor())
	{
		SetDissallowedReason(LOCTEXT("HLODInEditor_StreamingDisabled", "Streaming is disabled for this world"));
		return false;
	}

	if (!InWorld->GetWorldPartition()->IsHLODsInEditorAllowed())
	{
		SetDissallowedReason(LOCTEXT("HLODInEditor_HLODsInEditorDisallowed", "HLOD in editor is disabled for this world"));
		return false;
	}

	return true;
}

bool FWorldPartitionEditorModule::WriteHLODStats(const FWriteHLODStatsParams& Params) const
{
	return UWorldPartitionHLODEditorSubsystem::WriteHLODStats(Params);
}

void FWorldPartitionEditorModule::OnConvertMap()
{
	IContentBrowserSingleton& ContentBrowserSingleton = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get();
	
	FOpenAssetDialogConfig Config;
	Config.bAllowMultipleSelection = false;
	FString OutPathName;
	if (FPackageName::TryConvertFilenameToLongPackageName(FEditorDirectories::Get().GetLastDirectory(ELastDirectory::LEVEL), OutPathName))
	{
		Config.DefaultPath = OutPathName;
	}	
	Config.AssetClassNames.Add(UWorld::StaticClass()->GetClassPathName());

	TArray<FAssetData> Assets = ContentBrowserSingleton.CreateModalOpenAssetDialog(Config);
	if (Assets.Num() == 1)
	{
		ConvertMap(Assets[0].PackageName.ToString());
	}
}

static bool AskSaveDirtyPackages(const bool bSaveContentPackages)
{
	// Ask user to save dirty packages
	const bool bPromptUserToSave = true;
	const bool bSaveMapPackages = true;
	const bool bFastSave = false;
	const bool bNotifyNoPackagesSaved = false;
	const bool bCanBeDeclined = false;
	return FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages, bFastSave, bNotifyNoPackagesSaved, bCanBeDeclined);
}

static bool UnloadCurrentMap(FString& MapPackageName)
{
	UPackage* WorldPackage = FindPackage(nullptr, *MapPackageName);

	// Make sure we handle the case where the world package was renamed on save (for temp world for example)
	if (WorldPackage)
	{
		MapPackageName = WorldPackage->GetLoadedPath().GetPackageName();
	}

	// Unload any loaded map
	if (!UEditorLoadingAndSavingUtils::NewBlankMap(/*bSaveExistingMap*/false))
	{
		return false;
	}

	return true;
}

static void RescanAssets(const FString& MapToScan)
{
	// Force a directory watcher tick for the asset registry to get notified of the changes
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::Get().LoadModuleChecked<FDirectoryWatcherModule>(TEXT("DirectoryWatcher"));
	DirectoryWatcherModule.Get()->Tick(-1.0f);

	// Force update
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	TArray<FString> ExternalObjectsPaths = ULevel::GetExternalObjectsPaths(MapToScan);

	AssetRegistry.ScanModifiedAssetFiles({ MapToScan });
	AssetRegistry.ScanPathsSynchronous(ExternalObjectsPaths, true);
}

static void LoadMap(const FString& MapToLoad)
{
	FEditorFileUtils::LoadMap(MapToLoad);

	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (!World || World->GetPackage()->GetLoadedPath().GetPackageName() != MapToLoad)
	{
		UE_LOGF(LogWorldPartitionEditor, Error, "Failed to reopen world.");
	}
}

void FWorldPartitionEditorModule::RunCommandletAsExternalProcess(const FString& InCommandletArgs, const FText& InOperationDescription, int32& OutResult, bool& bOutCancelled)
{
	OutResult = 0;
	bOutCancelled = false;

	FProcHandle ProcessHandle;

	FScopedSlowTask SlowTask(1.0f, InOperationDescription);
	SlowTask.MakeDialog(true);

	void* ReadPipe = nullptr;
	void* WritePipe = nullptr;
	verify(FPlatformProcess::CreatePipe(ReadPipe, WritePipe));

	FString CurrentExecutableName = FPlatformProcess::ExecutablePath();

	// Try to provide complete Path, if we can't try with project name
	FString ProjectPath = FPaths::IsProjectFilePathSet() ? FPaths::GetProjectFilePath() : FApp::GetProjectName();

	// Obtain the log file path that will be used by the commandlet
	FString LogFilePrefix = TEXT("Commandlet");
	if (!FParse::Value(*InCommandletArgs, TEXT("Builder="), LogFilePrefix))
	{
		FParse::Value(*InCommandletArgs, TEXT("Run="), LogFilePrefix);
	}
	const FString TimeStamp = FString::Printf(TEXT("-%08x-%s"), FPlatformProcess::GetCurrentProcessId(), *FDateTime::Now().ToIso8601().Replace(TEXT(":"), TEXT(".")));
	const FString RelLogFilePath = FPaths::ProjectLogDir() / TEXT("WorldPartition") / LogFilePrefix + TimeStamp + TEXT(".log");
	const FString AbsLogFilePath = FPaths::ConvertRelativePathToFull(RelLogFilePath);
		
	TArray<FString> CommandletArgsArray;
	CommandletArgsArray.Add(TEXT("\"") + ProjectPath + TEXT('"'));
	CommandletArgsArray.Add(TEXT("-BaseDir=\"") + FString(FPlatformProcess::BaseDir()) + TEXT('"'));
	CommandletArgsArray.Add(TEXT("-Unattended"));
	CommandletArgsArray.Add(TEXT("-RunningFromUnrealEd"));
	CommandletArgsArray.Add(TEXT("-AbsLog=\"") + AbsLogFilePath + TEXT('"'));
	CommandletArgsArray.Add(InCommandletArgs);

	OnExecuteCommandletEvent.Broadcast(CommandletArgsArray);

	FString Arguments;
	for (const FString& AdditionalArg : CommandletArgsArray)
	{
		Arguments += " ";
		Arguments += AdditionalArg;
	}
	
	UE_LOGF(LogWorldPartitionEditor, Display, "Running commandlet: %ls %ls", *CurrentExecutableName, *Arguments);

	uint32 ProcessID;
	const bool bLaunchDetached = false;
	const bool bLaunchHidden = true;
	const bool bLaunchReallyHidden = true;
	ProcessHandle = FPlatformProcess::CreateProc(*CurrentExecutableName, *Arguments, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, &ProcessID, 0, nullptr, WritePipe);

	while (FPlatformProcess::IsProcRunning(ProcessHandle))
	{
		if (SlowTask.ShouldCancel() || GEditor->GetMapBuildCancelled())
		{
			bOutCancelled = true;
			FPlatformProcess::TerminateProc(ProcessHandle);
			break;
		}

		FString LogString = FPlatformProcess::ReadPipe(ReadPipe);

		// Parse output, look for progress indicator in the log (in the form "Display: [i / N] Msg...\n")
		const FRegexPattern LogProgressPattern(TEXT("Display:\\s\\[([0-9]+)\\s\\/\\s([0-9]+)\\]\\s(.+)?(?=\\.{3}$)"));
		FRegexMatcher Regex(LogProgressPattern, *LogString);
		while (Regex.FindNext())
		{
			// Update slow task progress & message
			SlowTask.CompletedWork = FCString::Atoi(*Regex.GetCaptureGroup(1));
			SlowTask.TotalAmountOfWork = FCString::Atoi(*Regex.GetCaptureGroup(2));
			SlowTask.DefaultMessage = FText::FromString(Regex.GetCaptureGroup(3));
		}

		SlowTask.EnterProgressFrame(0);
		FPlatformProcess::Sleep(0.1f);
	}

	FPlatformProcess::GetProcReturnCode(ProcessHandle, &OutResult);
	FPlatformProcess::ClosePipe(ReadPipe, WritePipe);

	if (OutResult == 0)
	{
		UE_LOGF(LogWorldPartitionEditor, Display, "Commandlet executed successfully.");
		UE_LOGF(LogWorldPartitionEditor, Display, "Detailed output can be found in %ls", *AbsLogFilePath);
	}
	else
	{
		UE_LOGF(LogWorldPartitionEditor, Error, "#### Commandlet Failed ####");
		UE_LOGF(LogWorldPartitionEditor, Error, "%ls %ls", *CurrentExecutableName, *Arguments);
		UE_LOGF(LogWorldPartitionEditor, Error, "Return Code: %i", OutResult);

		UE_LOGF(LogWorldPartitionEditor, Error, "#### BEGIN COMMANDLET OUTPUT (from %ls) ####", *AbsLogFilePath);

		TArray<FString> OutputLines;
		FFileHelper::LoadFileToStringArray(OutputLines, *AbsLogFilePath);
		for (const FString& OutputLine : OutputLines)
		{
			const FRegexPattern LogCategoryVerbosityPattern(TEXT("^(?:\\[.*\\])?\\w*:\\s(\\w*):\\s"));
			FRegexMatcher Regex(LogCategoryVerbosityPattern, *OutputLine);
			if (Regex.FindNext())
			{
				FString VerbosityString = Regex.GetCaptureGroup(1);
				ELogVerbosity::Type Verbosity = ParseLogVerbosityFromString(VerbosityString);
				switch (Verbosity)
				{
				case ELogVerbosity::Display: UE_LOGF(LogWorldPartitionEditor, Display, "#### COMMANDLET OUTPUT >> %ls", *OutputLine); break;
				case ELogVerbosity::Warning: UE_LOGF(LogWorldPartitionEditor, Warning, "#### COMMANDLET OUTPUT >> %ls", *OutputLine); break;
				case ELogVerbosity::Error:	 UE_LOGF(LogWorldPartitionEditor, Error, "  #### COMMANDLET OUTPUT >> %ls", *OutputLine); break;
				case ELogVerbosity::Fatal:	 UE_LOGF(LogWorldPartitionEditor, Error, "  #### COMMANDLET OUTPUT >> %ls", *OutputLine); break; // Do not output as FATAL as it would crash the editor
				default: break;	// Ignore the non displayable log lines, they can be found in the log file
				}
			}
		}		

		UE_LOGF(LogWorldPartitionEditor, Error, "#### END COMMANDLET OUTPUT ####");
	}
}

bool FWorldPartitionEditorModule::ConvertMap(const FString& InLongPackageName)
{
	if (ULevel::GetIsLevelPartitionedFromPackage(FName(*InLongPackageName)))
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ConvertMapMsg", "Map is already using World Partition"));
		return true;
	}

	UWorldPartitionConvertOptions* DefaultConvertOptions = GetMutableDefault<UWorldPartitionConvertOptions>();
	DefaultConvertOptions->CommandletClass = GetDefault<UWorldPartitionEditorSettings>()->GetCommandletClass();
	DefaultConvertOptions->bInPlace = false;
	DefaultConvertOptions->bSkipStableGUIDValidation = false;
	DefaultConvertOptions->LongPackageName = InLongPackageName;

	TSharedPtr<SWindow> DlgWindow =
		SNew(SWindow)
		.Title(LOCTEXT("ConvertWindowTitle", "Convert Settings"))
		.ClientSize(SWorldPartitionConvertDialog::DEFAULT_WINDOW_SIZE)
		.SizingRule(ESizingRule::UserSized)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.SizingRule(ESizingRule::FixedSize);

	TSharedRef<SWorldPartitionConvertDialog> ConvertDialog =
		SNew(SWorldPartitionConvertDialog)
		.ParentWindow(DlgWindow)
		.ConvertOptions(DefaultConvertOptions);

	DlgWindow->SetContent(ConvertDialog);

	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
	FSlateApplication::Get().AddModalWindow(DlgWindow.ToSharedRef(), MainFrameModule.GetParentWindow());

	if (ConvertDialog->ClickedOk())
	{
		// Ask user to save dirty packages
		if (!AskSaveDirtyPackages(/*bAskSaveContentPackages=*/false))
		{
			return false;
		}

		if (!UnloadCurrentMap(DefaultConvertOptions->LongPackageName))
		{
			return false;
		}

		const FString CommandletArgs = *DefaultConvertOptions->ToCommandletArgs();
		const FText OperationDescription = LOCTEXT("ConvertProgress", "Converting map to world partition...");
		
		int32 Result;
		bool bCancelled;
		RunCommandletAsExternalProcess(CommandletArgs, OperationDescription, Result, bCancelled);
		if (!bCancelled && Result == 0)
		{	
#if	PLATFORM_DESKTOP
			if (DefaultConvertOptions->bGenerateIni)
			{
				const FString PackageFilename = FPackageName::LongPackageNameToFilename(DefaultConvertOptions->LongPackageName);
				const FString PackageDirectory = FPaths::ConvertRelativePathToFull(FPaths::GetPath(PackageFilename));
				FPlatformProcess::ExploreFolder(*PackageDirectory);
			}
#endif				
				
			FString MapToLoad = DefaultConvertOptions->LongPackageName;
			if (!DefaultConvertOptions->bInPlace)
			{
				MapToLoad += UWorldPartitionConvertCommandlet::GetConversionSuffix(DefaultConvertOptions->bOnlyMergeSubLevels);
			}

			RescanAssets(MapToLoad);
			LoadMap(MapToLoad);
		}
		else if (bCancelled)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ConvertMapCancelled", "Conversion cancelled!"));
		}
		else if(Result != 0)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("ConvertMapFailed", "Conversion failed! See log for details."));
		}
	}

	return false;
}

bool IWorldPartitionEditorModule::RunBuilder(TSubclassOf<UWorldPartitionBuilder> InWorldPartitionBuilder, UWorld* InWorld)
{
	FRunBuilderParams Params;
	Params.BuilderClass = InWorldPartitionBuilder;
	Params.World = InWorld;
	return RunBuilder(Params);
}

bool FWorldPartitionEditorModule::RunBuilder(const FRunBuilderParams& InParams)
{
	// Ideally this should be improved to automatically register all builders & present their options in a consistent way...

	if (InParams.BuilderClass == UWorldPartitionHLODsBuilder::StaticClass())
	{
		return BuildHLODs(InParams);
	}
	
	if (InParams.BuilderClass == UWorldPartitionMiniMapBuilder::StaticClass())
	{
		return BuildMinimap(InParams);
	}

	if (InParams.BuilderClass == UWorldPartitionLandscapeSplineMeshesBuilder::StaticClass())
	{
		return BuildLandscapeSplineMeshes(InParams.World);
	}

	return Build(InParams);
}

bool FWorldPartitionEditorModule::BuildHLODs(const FRunBuilderParams& InParams)
{
	TSharedPtr<SWindow> DlgWindow =
		SNew(SWindow)
		.Title(LOCTEXT("BuildHLODsWindowTitle", "Build HLODs"))
		.ClientSize(SWorldPartitionBuildHLODsDialog::DEFAULT_WINDOW_SIZE)
		.SizingRule(ESizingRule::UserSized)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.SizingRule(ESizingRule::FixedSize);

	TSharedRef<SWorldPartitionBuildHLODsDialog> BuildHLODsDialog =
		SNew(SWorldPartitionBuildHLODsDialog)
		.ParentWindow(DlgWindow);

	DlgWindow->SetContent(BuildHLODsDialog);

	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
	FSlateApplication::Get().AddModalWindow(DlgWindow.ToSharedRef(), MainFrameModule.GetParentWindow());

	if (BuildHLODsDialog->GetDialogResult() != SWorldPartitionBuildHLODsDialog::DialogResult::Cancel)
	{
		FRunBuilderParams ParamsCopy(InParams);
		ParamsCopy.ExtraArgs = BuildHLODsDialog->GetDialogResult() == SWorldPartitionBuildHLODsDialog::DialogResult::BuildHLODs ? "-SetupHLODs -BuildHLODs -AllowCommandletRendering" : "-DeleteHLODs";
		ParamsCopy.OperationDescription = LOCTEXT("HLODBuildProgress", "Building HLODs...");

		return Build(ParamsCopy);
	}

	return false;
}


bool FWorldPartitionEditorModule::BuildMinimap(const FRunBuilderParams& InParams)
{
	FRunBuilderParams ParamsCopy(InParams);
	ParamsCopy.ExtraArgs = "-AllowCommandletRendering";
	ParamsCopy.OperationDescription = LOCTEXT("MinimapBuildProgress", "Building minimap...");
	return Build(ParamsCopy);
}

bool FWorldPartitionEditorModule::Build(const FRunBuilderParams& InParams)
{
	FRunBuilderParams ParamsCopy(InParams);
	OnPreExecuteCommandletEvent.Broadcast(ParamsCopy);

	UWorld* World = ParamsCopy.World;
	UPackage* WorldPackage = World->GetPackage();
	
	// Ask user to save dirty packages
	if (!AskSaveDirtyPackages(/*bAskSaveContentPackages=*/true))
	{			
		return false;
	}

	// Validate that a newly created world was actually saved
	if (WorldPackage->HasAnyPackageFlags(PKG_NewlyCreated))
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("NewMap", "New world must be saved before performing this operation."));
		return false;
	}
	
	// Unload map if required
	FString WorldPackageName = WorldPackage->GetName();
	if (!UnloadCurrentMap(WorldPackageName))
	{
		return false;
	}

	// Close assets editors as the external UE process may try to update those same assets
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllAssetEditors();

	TStringBuilder<512> CommandletArgsBuilder;
	CommandletArgsBuilder.Append(WorldPackageName);
	CommandletArgsBuilder.Append(" -run=WorldPartitionBuilderCommandlet -Builder=");
	CommandletArgsBuilder.Append(ParamsCopy.BuilderClass->GetName());
		
	if (!ParamsCopy.ExtraArgs.IsEmpty())
	{
		CommandletArgsBuilder.Append(" ");
		CommandletArgsBuilder.Append(ParamsCopy.ExtraArgs);
	}

	const FText OperationDescription = ParamsCopy.OperationDescription.IsEmptyOrWhitespace() ? LOCTEXT("BuildProgress", "Building...") : ParamsCopy.OperationDescription;

	int32 Result;
	bool bCancelled;

	FString CommandletArgs(CommandletArgsBuilder.ToString());
	RunCommandletAsExternalProcess(CommandletArgs, OperationDescription, Result, bCancelled);

	RescanAssets(WorldPackageName);
	LoadMap(WorldPackageName);

	if (bCancelled)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("BuildCancelled", "Build cancelled!"));
	}
	else if (Result != 0)
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("BuildFailed", "Build failed! See log for details."));
	}

	OnPostExecuteCommandletEvent.Broadcast();

	return !bCancelled && Result == 0;
}

bool FWorldPartitionEditorModule::BuildLandscapeSplineMeshes(UWorld* InWorld)
{
	if (!UWorldPartitionLandscapeSplineMeshesBuilder::RunOnInitializedWorld(InWorld))
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("LandscapeSplineMeshesBuildFailed", "Landscape Spline Meshes build failed! See log for details."));
		return false;
	}
	return true;
}

TSharedRef<SDockTab> FWorldPartitionEditorModule::SpawnWorldPartitionTab(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> NewTab =
		SNew(SDockTab)
		.Label(NSLOCTEXT("LevelEditor", "WorldBrowserPartitionTabTitle", "World Partition"))
		[
			CreateWorldPartitionEditor()
		];

	WorldPartitionTab = NewTab;
	return NewTab;
}

TSharedRef<SDockTab> FWorldPartitionEditorModule::SpawnContentBundleTab(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> NewTab =
		SNew(SDockTab)
		.Label(NSLOCTEXT("LevelEditor", "ContentBundleTabTitle", "Content Bundles"))
		[
			CreateContentBundleBrowser()
		];

	ContentBundleTab = NewTab;
	return NewTab;
}

void FWorldPartitionEditorModule::RegisterWorldPartitionTabs(TSharedPtr<FTabManager> InTabManager)
{
	const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();

	const FSlateIcon WorldPartitionIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.WorldPartition");

	InTabManager->RegisterTabSpawner(WorldPartitionEditorTabId,
		FOnSpawnTab::CreateRaw(this, &FWorldPartitionEditorModule::SpawnWorldPartitionTab))
		.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "WorldPartitionEditor", "World Partition Editor"))
		.SetTooltipText(NSLOCTEXT("LevelEditorTabs", "WorldPartitionEditorTooltipText", "Open the World Partition Editor."))
		.SetGroup(MenuStructure.GetLevelEditorWorldPartitionCategory())
		.SetIcon(WorldPartitionIcon);

	constexpr TCHAR PLACEHOLDER_ContentBundleIcon[] = TEXT("LevelEditor.Tabs.DataLayers"); // todo_ow: create placeholder icon for content bundle tab
	const FSlateIcon DataLayersIcon(FAppStyle::GetAppStyleSetName(), PLACEHOLDER_ContentBundleIcon);
	InTabManager->RegisterTabSpawner(ContentBundleBrowserTabId,
		FOnSpawnTab::CreateRaw(this, &FWorldPartitionEditorModule::SpawnContentBundleTab))
		.SetDisplayName(NSLOCTEXT("LevelEditorTabs", "LevelEditorContentBundleBrowser", "Content Bundles Outliner"))
		.SetTooltipText(NSLOCTEXT("LevelEditorTabs", "LevelEditorContentBundleBrowserTooltipText", "Open the Content Bundles Outliner."))
		.SetGroup(MenuStructure.GetLevelEditorWorldPartitionCategory())
		.SetIcon(DataLayersIcon);
}

void FWorldPartitionEditorModule::RegisterWorldPartitionLayout(FLayoutExtender& Extender)
{
	Extender.ExtendLayout(FTabId("LevelEditorSelectionDetails"), ELayoutExtensionPosition::After, FTabManager::FTab(WorldPartitionEditorTabId, ETabState::ClosedTab));
}

static bool ShouldRepairWorldDataLayers(UWorld* World)
{
	AWorldDataLayers* WorldDataLayers = World ? World->PersistentLevel->GetWorldDataLayers() : nullptr;
	if (WorldDataLayers &&
		WorldDataLayers->GetPackage()->HasAnyPackageFlags(PKG_NewlyCreated) &&
		!World->GetPackage()->HasAnyPackageFlags(PKG_NewlyCreated) &&
		!FPackageName::DoesPackageExist(WorldDataLayers->GetPackage()->GetName()) &&
		FPackageName::DoesPackageExist(World->GetPackage()->GetName()))
	{
		return true;
	}
	return false;
}

bool FWorldPartitionEditorModule::HasErrors(UWorld* World) const
{
	if (UWorldPartition* const WorldPartition = World ? World->GetWorldPartition() : nullptr)
	{
		if (ShouldRepairWorldDataLayers(World))
		{
			return true;
		}

		bool bHasErrors = false;

		WorldPartition->ForEachActorDescContainerInstanceBreakable([&bHasErrors](UActorDescContainerInstance* ActorDescContainerInstance)
		{
			if (ActorDescContainerInstance->GetContainer()->HasInvalidActors())
			{
				bHasErrors = true;
			}
			return !bHasErrors;
		});

		return bHasErrors;
	}

	return false;
}

void FWorldPartitionEditorModule::RepairErrors(UWorld* World)
{
	if (UWorldPartition* const WorldPartition = World ? World->GetWorldPartition() : nullptr)
	{
		if (ShouldRepairWorldDataLayers(World))
		{
			TSet<UPackage*> PackagesToSave;
			PackagesToSave.Add(World->PersistentLevel->GetWorldDataLayers()->GetPackage());
			PackagesToSave.Add(World->GetPackage());
			if (PackagesToSave.Num())
			{
				FEditorFileUtils::FPromptForCheckoutAndSaveParams SaveParams;
				SaveParams.bCheckDirty = false;
				SaveParams.bPromptToSave = false;
				SaveParams.bIsExplicitSave = true;
				FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave.Array(), SaveParams);
			}
		}

		ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
		ISourceControlProvider& SourceControlProvider = SourceControlModule.GetProvider();

		TArray<FAssetData> InvalidActorAssets;
		WorldPartition->ForEachActorDescContainerInstance([&InvalidActorAssets](UActorDescContainerInstance* ActorDescContainerInstance)
		{
			for (const FAssetData& InvalidActor : ActorDescContainerInstance->GetContainer()->GetInvalidActors())
			{
				InvalidActorAssets.Add(InvalidActor);
			}
			ActorDescContainerInstance->GetContainer()->ClearInvalidActors();
		});

		FlushAsyncLoading();

		TArray<FString> ActorFilesToDelete;
		TArray<FString> ActorFilesToRevert;
		{
			FScopedSlowTask SlowTask(InvalidActorAssets.Num(), LOCTEXT("UpdatingSourceControlStatus", "Updating source control status..."));
			SlowTask.MakeDialogDelayed(1.0f);

			for (const FAssetData& InvalidActorAsset : InvalidActorAssets)
			{
				FPackagePath PackagePath;
				if (FPackagePath::TryFromPackageName(InvalidActorAsset.PackageName, PackagePath))
				{
					if (UPackage* ExistingPackage = FindObject<UPackage>(nullptr, *InvalidActorAsset.PackageName.ToString()))
					{
						ResetLoaders(ExistingPackage);
					}

					const FString ActorFile = PackagePath.GetLocalFullPath();
					FSourceControlStatePtr SCState = SourceControlProvider.GetState(ActorFile, EStateCacheUsage::ForceUpdate);

					if (SCState.IsValid() && SCState->IsSourceControlled())
					{
						if (SCState->IsAdded())
						{
							ActorFilesToRevert.Add(ActorFile);
						}
						else
						{
							if (SCState->IsCheckedOut())
							{
								ActorFilesToRevert.Add(ActorFile);
							}

							ActorFilesToDelete.Add(ActorFile);
						}
					}
					else
					{
						IFileManager::Get().Delete(*ActorFile, false, true);
					}
				}

				SlowTask.EnterProgressFrame(1);
			}
		}

		if (ActorFilesToRevert.Num() || ActorFilesToDelete.Num())
		{
			FScopedSlowTask SlowTask(ActorFilesToRevert.Num() + ActorFilesToDelete.Num(), LOCTEXT("DeletingFiles", "Deleting files..."));
			SlowTask.MakeDialogDelayed(1.0f);

			if (ActorFilesToRevert.Num())
			{
				SourceControlProvider.Execute(ISourceControlOperation::Create<FRevert>(), ActorFilesToRevert);
				SlowTask.EnterProgressFrame(ActorFilesToRevert.Num());
			}

			if (ActorFilesToDelete.Num())
			{
				SourceControlProvider.Execute(ISourceControlOperation::Create<FDelete>(), ActorFilesToDelete);
				SlowTask.EnterProgressFrame(ActorFilesToDelete.Num());
			}
		}
	}
}

FString UWorldPartitionConvertOptions::ToCommandletArgs() const
{
	TStringBuilder<1024> CommandletArgsBuilder;
	CommandletArgsBuilder.Appendf(TEXT("-run=%s %s -AllowCommandletRendering"), *CommandletClass->GetName(), *LongPackageName);
	
	if (!bInPlace)
	{
		CommandletArgsBuilder.Append(TEXT(" -ConversionSuffix"));
	}

	if (bSkipStableGUIDValidation)
	{
		CommandletArgsBuilder.Append(TEXT(" -SkipStableGUIDValidation"));
	}

	if (bDeleteSourceLevels)
	{
		CommandletArgsBuilder.Append(TEXT(" -DeleteSourceLevels"));
	}
	
	if (bGenerateIni)
	{
		CommandletArgsBuilder.Append(TEXT(" -GenerateIni"));
	}
	
	if (bReportOnly)
	{
		CommandletArgsBuilder.Append(TEXT(" -ReportOnly"));
	}
	
	if (bVerbose)
	{
		CommandletArgsBuilder.Append(TEXT(" -Verbose"));
	}

	if (bOnlyMergeSubLevels)
	{
		CommandletArgsBuilder.Append(TEXT(" -OnlyMergeSubLevels"));
	}

	if (bSaveFoliageTypeToContentFolder)
	{
		CommandletArgsBuilder.Append(TEXT(" -FoliageTypePath=/Game/FoliageTypes"));
	}
	
	return CommandletArgsBuilder.ToString();
}

#undef LOCTEXT_NAMESPACE
