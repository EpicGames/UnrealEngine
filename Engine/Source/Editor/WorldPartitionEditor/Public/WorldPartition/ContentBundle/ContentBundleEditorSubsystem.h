// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ActorEditorContextState.h"
#include "CoreMinimal.h"
#include "EditorSubsystem.h"
#include "IActorEditorContextClient.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "WorldPartition/ContentBundle/ContentBundleEditorSubsystemInterface.h"
#include "ContentBundleEditorSubsystem.generated.h"

#define UE_API WORLDPARTITIONEDITOR_API

class FContentBundleEditor;
class UContentBundleDescriptor;
class UContentBundleEditorSubsystem;

UCLASS(Within = ContentBundleEditorSubsystem)
class UContentBundleEditorSubsystemModule : public UObject
{
	GENERATED_BODY()

public:
	//~ Begin UObject interface
	virtual void BeginDestroy() override { check(!IsInitialize()); Super::BeginDestroy(); }
	//~ End UObject interface

	void Initialize() { DoInitialize(); bIsInitialized = true; }
	void Deinitialize() { bIsInitialized = false; DoDenitialize(); }

	bool IsInitialize() const { return bIsInitialized; }

	UContentBundleEditorSubsystem* GetSubsystem() const { return GetOuterUContentBundleEditorSubsystem(); }

protected:
	virtual void DoInitialize() {};
	virtual void DoDenitialize() {};

private:
	bool bIsInitialized = false;
};

UCLASS()
class UActorEditorContextContentBundleState : public UActorEditorContextClientState
{
	GENERATED_BODY()

public:
	UPROPERTY(VisibleAnywhere, Category = "Content Bundle")
	FGuid ContentBundleGuid;
};

UCLASS()
class UContentBundleEditingSubmodule : public UContentBundleEditorSubsystemModule, public IActorEditorContextClient
{
	GENERATED_BODY()

	friend class UContentBundleEditorSubsystem;

public:
	//~ Begin IActorEditorContextClient interface
	virtual void OnExecuteActorEditorContextAction(UWorld* InWorld, const EActorEditorContextAction& InType, AActor* InActor = nullptr) override;
	virtual void CaptureActorEditorContextState(UWorld* InWorld, UActorEditorContextStateCollection* InStateCollection) const override;
	virtual void RestoreActorEditorContextState(UWorld* InWorld, const UActorEditorContextStateCollection* InStateCollection) override;
	virtual bool GetActorEditorContextDisplayInfo(UWorld* InWorld, FActorEditorContextClientDisplayInfo& OutDiplayInfo) const override;
	virtual bool CanResetContext(UWorld* InWorld) const override { return true; }
	virtual TSharedRef<SWidget> GetActorEditorContextWidget(UWorld* InWorld) const override;
	virtual FOnActorEditorContextClientChanged& GetOnActorEditorContextClientChanged() override { return ActorEditorContextClientChanged; }
	//~ End IActorEditorContextClient interface

	bool ActivateContentBundleEditing(TSharedPtr<FContentBundleEditor>& ContentBundleEditor);
	bool DeactivateContentBundleEditing(TSharedPtr<FContentBundleEditor>& ContentBundleEditor);
	bool DeactivateCurrentContentBundleEditing();
	bool IsEditingContentBundle() const;
	bool IsEditingContentBundle(const TSharedPtr<FContentBundleEditor>& ContentBundleEditor) const;
	bool IsEditingContentBundle(const FGuid& ContentBundleGuid) const;
	TSharedPtr<FContentBundleEditor> GetEditorContentBundle(const FGuid& ContentBundleGuid) const;

protected:
	//~ Begin UContentBundleEditorSubsystemModule interface
	virtual void DoInitialize() override;
	virtual void DoDenitialize() override;
	//~ End UContentBundleEditorSubsystemModule interface

private:
	void PushContentBundleEditing(bool bDuplicateContext);
	void PopContentBundleEditing();
	void StartEditing(TSharedPtr<FContentBundleEditor>& ContentBundleEditor);
	void StopEditing(TSharedPtr<FContentBundleEditor>& ContentBundleEditor);
	void ApplyContext(AActor* InActor);
	virtual void PreEditUndo() override;
	virtual void PostEditUndo() override;

	UPROPERTY()
	TArray<FGuid> EditingContentBundlesStack;

	UPROPERTY()
	FGuid EditingContentBundleGuid;

	// Used for undo/redo
	FGuid PreUndoRedoEditingContentBundleGuid;

	FOnActorEditorContextClientChanged ActorEditorContextClientChanged;
};

UCLASS(MinimalAPI)
class UContentBundleEditorSubsystem : public UEditorSubsystem, public IContentBundleEditorSubsystemInterface
{
	GENERATED_BODY()

public:
	static UContentBundleEditorSubsystem* Get() { return StaticCast<UContentBundleEditorSubsystem*>(IContentBundleEditorSubsystemInterface::Get()); }

	UContentBundleEditorSubsystem() {}

	//~ Begin UEditorSubsystem interface
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	//~ End UEditorSubsystem interface

	//~ Begin IContentBundleEditorSubsystemInterface interface
	virtual void NotifyContentBundleAdded(const FContentBundleEditor* ContentBundle) override { OnContentBundleAdded().Broadcast(ContentBundle); }
	virtual void NotifyContentBundleRemoved(const FContentBundleEditor* ContentBundle) override { OnContentBundleRemoved().Broadcast(ContentBundle); }
	UE_API virtual void NotifyContentBundleInjectedContent(const FContentBundleEditor* ContentBundle) override;
	UE_API virtual void NotifyContentBundleRemovedContent(const FContentBundleEditor* ContentBundle) override;
	virtual void NotifyContentBundleChanged(const FContentBundleEditor* ContentBundle) override { OnContentBundleChanged().Broadcast(ContentBundle); }
	//~ End IContentBundleEditorSubsystemInterface interface

	UE_API UWorld* GetWorld() const;

	UContentBundleEditingSubmodule* GetEditingSubmodule() { return ContentBundleEditingSubModule; }

	UE_API TSharedPtr<FContentBundleEditor> GetEditorContentBundleForActor(const AActor* Actor);

	UE_API TArray<TSharedPtr<FContentBundleEditor>> GetEditorContentBundles();
	UE_API TSharedPtr<FContentBundleEditor> GetEditorContentBundle(const UContentBundleDescriptor* ContentBundleDescriptor) const;
	UE_API TSharedPtr<FContentBundleEditor> GetEditorContentBundle(const FGuid& ContentBundleGuid) const override;

	UE_API void SelectActors(FContentBundleEditor& EditorContentBundle);
	UE_API void DeselectActors(FContentBundleEditor& EditorContentBundle);

	UE_API void ReferenceAllActors(FContentBundleEditor& EditorContentBundle);
	UE_API void UnreferenceAllActors(FContentBundleEditor& EditorContentBundle);

	UE_API bool IsEditingContentBundle() const override;
	UE_API bool IsEditingContentBundle(const FGuid& ContentBundleGuid) const;
	UE_API bool ActivateContentBundleEditing(TSharedPtr<FContentBundleEditor>& ContentBundleEditor) const override;
	UE_API bool DeactivateContentBundleEditing(TSharedPtr<FContentBundleEditor>& ContentBundleEditor) const override;
	UE_API virtual bool DeactivateCurrentContentBundleEditing() const override;
	UE_API bool IsContentBundleEditingActivated(TSharedPtr<FContentBundleEditor>& ContentBundleEditor) const;

	UE_API virtual void PushContentBundleEditing(bool bDuplicateContext) override;
	UE_API virtual void PopContentBundleEditing() override;

private:

	FDelegateHandle LevelExternalActorsPathsProviderDelegateHandle;
	void OnGetLevelExternalActorsPaths(const FString& InLevelPackageName, const FString& InPackageShortName, TArray<FString>& OutExternalActorsPaths);

	UE_API void SelectActorsInternal(FContentBundleEditor& EditorContentBundle, bool bSelect);

	UPROPERTY()
	TObjectPtr<UContentBundleEditingSubmodule> ContentBundleEditingSubModule;
};

#undef UE_API
