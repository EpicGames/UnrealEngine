// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SceneOutlinerFwd.h"
#include "ISceneOutliner.h"
#include "ISceneOutlinerColumn.h"
#include "Widgets/Views/SHeaderRow.h"
#include "SceneOutlinerGutter.h"

class FContentBundleEditor;

template<typename ItemType> class STableRow;
class FContentBundleOutlinerClientColumn : public ISceneOutlinerColumn
{
public:
	FContentBundleOutlinerClientColumn(ISceneOutliner& SceneOutliner) : WeakSceneOutliner(StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared())) {}
	virtual ~FContentBundleOutlinerClientColumn() = default;
	static FName GetID();

	//////////////////////////////////////////////////////////////////////////
	// Begin ISceneOutlinerColumn Implementation
	virtual FName GetColumnID() override { return GetID(); }
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;
	virtual bool SupportsSorting() const override { return true; }
	virtual void SortItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems, const EColumnSortMode::Type SortMode) const override;
	// End ISceneOutlinerColumn Implementation
	//////////////////////////////////////////////////////////////////////////

private:
	FText GetClientDisplayName(const TWeakPtr<FContentBundleEditor>& ContentBundleEditor) const;

	TWeakPtr<ISceneOutliner> WeakSceneOutliner;
};
