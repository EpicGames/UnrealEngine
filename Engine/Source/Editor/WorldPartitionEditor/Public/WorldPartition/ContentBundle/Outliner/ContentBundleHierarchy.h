// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISceneOutlinerHierarchy.h"

class FContentBundleMode;
class FContentBundleEditor;

class FContentBundleHiearchy : public ISceneOutlinerHierarchy
{
public:
	static TUniquePtr<FContentBundleHiearchy> Create(FContentBundleMode* Mode);

	virtual ~FContentBundleHiearchy();

	//~ Begin ISceneOutlinerHierarchy interface
	virtual void CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const override;
	virtual void CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutChildren) const override;
	virtual FSceneOutlinerTreeItemPtr FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate = false) override;
	//~ End ISceneOutlinerHierarchy interface

private:
	void OnContentBundleChanged(const FContentBundleEditor* ContentBundle);
	void OnContentBundleAdded(const FContentBundleEditor* ContentBundle);
	void OnContentBundleRemoved(const FContentBundleEditor* ContentBundle);

	FContentBundleHiearchy(FContentBundleMode* Mode);
};