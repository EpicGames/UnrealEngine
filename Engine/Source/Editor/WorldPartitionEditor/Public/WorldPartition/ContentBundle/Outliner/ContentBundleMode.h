// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISceneOutlinerMode.h"

class SSceneOutliner;
class FContentBundleEditor;
class FUICommandList;

struct FContentBundleModeCreationParams
{
	FContentBundleModeCreationParams(SSceneOutliner* InSceneOutliner)
		:SceneOutliner(InSceneOutliner)
	{}

	SSceneOutliner* SceneOutliner;
};


class FContentBundleMode : public ISceneOutlinerMode
{
public:
	FContentBundleMode(const FContentBundleModeCreationParams& Params);
	~FContentBundleMode();

	//~ Begin ISceneOutlinerMode interface
	virtual void Rebuild() override;
	virtual TSharedPtr<SWidget> CreateContextMenu() override;

	virtual void OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item) override;
	virtual FReply OnKeyDown(const FKeyEvent& InKeyEvent) override;
	//~ Begin ISceneOutlinerMode interface

	UWorld* GetEditingWorld() const;

protected:
	//~ Begin ISceneOutlinerMode interface
	virtual TUniquePtr<ISceneOutlinerHierarchy> CreateHierarchy() override;
	//~ End ISceneOutlinerMode interface

	TWeakPtr<FContentBundleEditor> GetSelectedContentBundle(SSceneOutliner* InSceneOutliner) const;
	TSharedPtr<FContentBundleEditor> GetSelectedContentBundlePin(SSceneOutliner* InSceneOutliner) const;
	void ToggleContentBundleActivation(const TWeakPtr<FContentBundleEditor>& ContentBundle);

	void RegisterContextMenu();
	void UnregisterContextMenu();

private:

	/** Delegate to handle "Find in Content Browser" context menu option */
	void FindInContentBrowser();

	/** Delegate to handle enabling the "Find in Content Browser" context menu option */
	bool CanFindInContentBrowser() const;

	TSharedPtr<FUICommandList> Commands;
};