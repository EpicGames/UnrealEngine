// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "SceneOutlinerFwd.h"
#include "ISceneOutlinerTreeItem.h"

class FContentBundleEditor;
class FContentBundleMode;

struct FContentBundleTreeItem : ISceneOutlinerTreeItem
{
public:
	struct FInitializationValues
	{
		FInitializationValues(const TWeakPtr<FContentBundleEditor>& InContentBundleEditor, const FContentBundleMode& InMode)
			:ContentBundleEditor(InContentBundleEditor),
			Mode(InMode)
		{

		}

		const TWeakPtr<FContentBundleEditor> ContentBundleEditor;
		const FContentBundleMode& Mode;
	};

	FContentBundleTreeItem(const FInitializationValues& InitializationValues);
	TWeakPtr<FContentBundleEditor> GetContentBundleEditor() const { return IsValid() ? ContentBundleEditor : nullptr; }
	TSharedPtr<FContentBundleEditor> GetContentBundleEditorPin() const { return IsValid() ? ContentBundleEditor.Pin() : nullptr; }

	/* Begin ISceneOutlinerTreeItem Implementation */
	virtual bool IsValid() const override;
	virtual FSceneOutlinerTreeItemID GetID() const override;
	virtual FString GetDisplayString() const override;
	virtual bool CanInteract() const override;
	virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override;
	/* End ISceneOutlinerTreeItem Implementation */

	static const FSceneOutlinerTreeItemType Type;

	DECLARE_DELEGATE_RetVal_OneParam(bool, FFilterPredicate, const FContentBundleEditor*);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FInteractivePredicate, const FContentBundleEditor*);

	bool Filter(FFilterPredicate Pred) const
	{
		return Pred.Execute(GetContentBundleEditorPin().Get());
	}

	bool GetInteractiveState(FInteractivePredicate Pred) const
	{
		return Pred.Execute(GetContentBundleEditorPin().Get());
	}

	FSlateColor GetItemColor() const;

private:
	TWeakPtr<FContentBundleEditor> ContentBundleEditor;

	const FContentBundleMode& Mode;
};