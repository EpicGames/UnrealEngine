// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SSceneOutliner.h"

class FContentBundleEditor;

class SContentBundleOutliner : public SSceneOutliner
{
public:
	void Construct(const FArguments& InArgs, const FSceneOutlinerInitializationOptions& InitOptions)
	{
		SSceneOutliner::Construct(InArgs, InitOptions);
	}

	SContentBundleOutliner() {}
	virtual ~SContentBundleOutliner() = default;

	void SelectContentBundle(const TWeakPtr<FContentBundleEditor>& ContentBundle);
};
