// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

class SContentBundleOutliner;
class FContentBundleEditor;
class SVerticalBox;

class SContentBundleBrowser : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SContentBundleBrowser) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void SelectContentBundle(const TWeakPtr<FContentBundleEditor>& ContentBundle);

private:
	TSharedPtr<SContentBundleOutliner> ContentBundleOutliner;
	TSharedPtr<SVerticalBox> ContentAreaBox;
};
