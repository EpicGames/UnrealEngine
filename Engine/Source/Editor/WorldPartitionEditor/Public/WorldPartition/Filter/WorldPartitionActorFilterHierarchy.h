// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISceneOutlinerHierarchy.h"

class FWorldPartitionActorFilterMode;

class FWorldPartitionActorFilterHierarchy : public ISceneOutlinerHierarchy
{
public:
	static TUniquePtr<FWorldPartitionActorFilterHierarchy> Create(FWorldPartitionActorFilterMode* Mode);

	virtual ~FWorldPartitionActorFilterHierarchy();

	//~ Begin ISceneOutlinerHierarchy interface
	virtual void CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const override;
	virtual void CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutItems) const override;
	virtual FSceneOutlinerTreeItemPtr FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate = false) override;
	//~ End ISceneOutlinerHierarchy interface

private:
	virtual void CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutItems, bool bRecursive) const;
	FWorldPartitionActorFilterHierarchy(FWorldPartitionActorFilterMode* Mode);
};