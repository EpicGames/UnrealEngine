// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISceneOutlinerMode.h"
#include "WorldPartition/Filter/WorldPartitionActorFilter.h"
#include "Containers/Map.h"
#include "Templates/Tuple.h"

#define UE_API WORLDPARTITIONEDITOR_API

class SSceneOutliner;

class FWorldPartitionActorFilterMode : public ISceneOutlinerMode
{
public:
	// struct holding multi selection values that might be null if they differ between actors
	class FFilter
	{
	public:
		UE_API FFilter(TSharedPtr<FWorldPartitionActorFilter> LevelFilter, const TArray<const FWorldPartitionActorFilter*>& SelectedFilters);
		
		UE_API void Apply(FWorldPartitionActorFilter* Result) const;
				
		struct FDataLayerFilter
		{
			FDataLayerFilter(){}
			FDataLayerFilter(bool bInOverride, bool bInIncluded) : bOverride(bInOverride), bIncluded(bInIncluded) {}

			TOptional<bool> bOverride;
			TOptional<bool> bIncluded;
		};

		using FDataLayerFilters = TMap<FSoftObjectPath, FDataLayerFilter>;
	private:
		UE_API void Initialize(const FWorldPartitionActorFilter* Filter);
		UE_API void Override(const FWorldPartitionActorFilter* Other);
		UE_API void Merge(const FWorldPartitionActorFilter* Other);
				
		// Unmodified Reference Level Filter
		TSharedPtr<FWorldPartitionActorFilter> LevelFilter;
		// Current values for Filter based Selection		
		TMap<const FWorldPartitionActorFilter*, FDataLayerFilters> FilterValues;

		friend class FWorldPartitionActorFilterMode;
	};

	UE_API FWorldPartitionActorFilterMode(SSceneOutliner* InSceneOutliner, TSharedPtr<FFilter> InFilter);
	
	//~ Begin ISceneOutlinerMode interface
	UE_API virtual void Rebuild() override;
	UE_API virtual int32 GetTypeSortPriority(const ISceneOutlinerTreeItem& Item) const override;
	//~ Begin ISceneOutlinerMode interface
		
	UE_API void Apply(FWorldPartitionActorFilter& InOutResult) const;
	
	FFilter::FDataLayerFilters& FindChecked(const FWorldPartitionActorFilter* InFilter) const
	{
		return Filter->FilterValues.FindChecked(InFilter);
	}

	const FWorldPartitionActorFilter* GetFilter() const
	{
		return Filter->LevelFilter.Get();
	}

protected:
	//~ Begin ISceneOutlinerMode interface
	UE_API virtual TUniquePtr<ISceneOutlinerHierarchy> CreateHierarchy() override;
	//~ End ISceneOutlinerMode interface
		
private:
	TSharedPtr<FFilter> Filter;
};

#undef UE_API
