// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Internationalization/Text.h"
#include "PropertyEditorModule.h"
#include "Templates/SharedPointer.h"
#include "WorldPartition/Filter/WorldPartitionActorFilterMode.h"

#define UE_API WORLDPARTITIONEDITOR_API

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;
class ISceneOutliner;

// Registered (FLevelInstanceEditorModule::StartupModule) Property Customization for properties of type FWorldPartitionActorFilter
struct FWorldPartitionActorFilterPropertyTypeCustomization : public IPropertyTypeCustomization
{
public:
	UE_API ~FWorldPartitionActorFilterPropertyTypeCustomization();

	UE_API virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}

private:
	virtual TSharedPtr<FWorldPartitionActorFilterMode::FFilter> CreateModeFilter(TArray<UObject*> OuterObjects) = 0;
	virtual void ApplyFilter(TSharedRef<IPropertyHandle> PropertyHandle, const FWorldPartitionActorFilterMode& Mode) = 0;

	FDelegateHandle WorldPartitionActorFilterChangedHandle;
	TSharedPtr<ISceneOutliner> SceneOutliner;
};

#undef UE_API
