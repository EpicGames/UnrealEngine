// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "SceneOutlinerFwd.h"
#include "ISceneOutlinerTreeItem.h"

struct FWorldPartitionActorFilter;
class UClass;

struct FWorldPartitionActorFilterItemBase : ISceneOutlinerTreeItem
{
	FWorldPartitionActorFilterItemBase(FSceneOutlinerTreeItemType InType) : ISceneOutlinerTreeItem(InType) {}
	virtual UClass* GetIconClass() const = 0;
	virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override;
	virtual bool CanInteract() const override { return true; }

	static const FSceneOutlinerTreeItemType Type;
};

struct FWorldPartitionActorFilterItem : FWorldPartitionActorFilterItemBase
{
public:
	struct FTreeItemData
	{
		FTreeItemData(const FWorldPartitionActorFilter* InFilter)
			: Filter(InFilter)
		{}

		const FWorldPartitionActorFilter* Filter;
	};

	FWorldPartitionActorFilterItem(const FTreeItemData& InData);

	static FSceneOutlinerTreeItemID ComputeTreeItemID(const FWorldPartitionActorFilterItem::FTreeItemData& InData);

	/* Begin ISceneOutlinerTreeItem Implementation */
	virtual bool IsValid() const override { return Data.Filter != nullptr; }
	virtual FSceneOutlinerTreeItemID GetID() const override;
	virtual FString GetDisplayString() const override;
	/* End ISceneOutlinerTreeItem Implementation */

	virtual UClass* GetIconClass() const override;
	const FWorldPartitionActorFilter* GetFilter() const { return Data.Filter; }

	static const FSceneOutlinerTreeItemType Type;
private:
	FTreeItemData Data;
};

struct FWorldPartitionActorFilterDataLayerItem : FWorldPartitionActorFilterItemBase
{
public:
	struct FTreeItemData
	{
		FTreeItemData(const FWorldPartitionActorFilter* InFilter, const FSoftObjectPath& InAssetPath)
			: Filter(InFilter), AssetPath(InAssetPath)
		{}

		const FWorldPartitionActorFilter* Filter;
		const FSoftObjectPath AssetPath;
	};

	FWorldPartitionActorFilterDataLayerItem(const FTreeItemData& InData);

	static FSceneOutlinerTreeItemID ComputeTreeItemID(const FWorldPartitionActorFilterDataLayerItem::FTreeItemData& InData);

	/* Begin ISceneOutlinerTreeItem Implementation */
	virtual bool IsValid() const override { return Data.Filter != nullptr; }
	virtual FSceneOutlinerTreeItemID GetID() const override;
	virtual FString GetDisplayString() const override;
	/* End ISceneOutlinerTreeItem Implementation */
	
	virtual UClass* GetIconClass() const override;
	const FWorldPartitionActorFilter* GetFilter() const { return Data.Filter; }
	const FSoftObjectPath& GetAssetPath() const { return Data.AssetPath; }
	bool GetDefaultValue() const;

	static const FSceneOutlinerTreeItemType Type;
private:
	FTreeItemData Data;
};