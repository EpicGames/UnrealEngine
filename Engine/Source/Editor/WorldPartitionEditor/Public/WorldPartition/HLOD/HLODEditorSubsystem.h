// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "Templates/PimplPtr.h"
#include "Tickable.h"
#include "UObject/ObjectKey.h"
#include "Engine/World.h"
#include "WorldPartition/IWorldPartitionEditorModule.h"
#include "WorldPartition/WorldPartitionActorLoaderInterface.h"

#include "HLODEditorSubsystem.generated.h"

#define UE_API WORLDPARTITIONEDITOR_API


class AActor;
class AWorldPartitionHLOD;
class ILevelEditor;
class UPrimitiveComponent;
class UWorldPartition;
class UWorldPartitionEditorSettings;
struct FWorldPartitionHLODEditorData;

// Visibility level for HLOD settings
// By default, settings are classified in the "AllSettings" category
enum class EHLODSettingsVisibility : uint8
{
	BasicSettings,
	AllSettings
};


/**
 * UWorldPartitionHLODEditorSubsystem
 */
UCLASS(MinimalAPI)
class UWorldPartitionHLODEditorSubsystem : public UEditorSubsystem, public FTickableGameObject
{
	GENERATED_BODY()

public:
	UE_API UWorldPartitionHLODEditorSubsystem();
	UE_API virtual ~UWorldPartitionHLODEditorSubsystem();

	//~ Begin USubsystem Interface.
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	//~ End USubsystem Interface.

	//~ Begin FTickableGameObject Interface
	virtual bool IsTickable() const override;
	virtual bool IsTickableInEditor() const override { return true; }
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual TStatId GetStatId() const override;
	//~ End FTickableGameObject Interface

	static UE_API bool WriteHLODStats(const IWorldPartitionEditorModule::FWriteHLODStatsParams& Params);

	static UE_API void AddHLODSettingsFilter(EHLODSettingsVisibility InSettingsVisibility, TSoftObjectPtr<UStruct> InStruct, FName InPropertyName);

	UE_API void ForceHLODVisibilityUpdate();
	
private:
	UE_API bool IsHLODInEditorEnabled();
	UE_API void SetHLODInEditorEnabled(bool bInEnable);

	UE_API void OnWorldPartitionInitialized(UWorldPartition* InWorldPartition);
	UE_API void OnWorldPartitionUninitialized(UWorldPartition* InWorldPartition);

	UE_API void OnLoaderAdapterStateChanged(const IWorldPartitionActorLoaderInterface::ILoaderAdapter* LoaderAdapter);

	UE_API void ForceHLODStateUpdate();

	static UE_API bool WriteHLODStats(UWorld* InWorld, const FString& InFilename);
	static UE_API bool WriteHLODInputStats(UWorld* InWorld, const FString& InFilename);
	
	UE_API void OnWorldPartitionEditorSettingsChanged(const FName& PropertyName, const UWorldPartitionEditorSettings& WorldPartitionEditorSettings);
	UE_API void ApplyHLODSettingsFiltering();

	UE_API void OnColorHandlerPropertyChangedEvent(UObject* InObject, FPropertyChangedEvent& InPropertyChangedEvent);

	void OnHLODActorRegistered(AWorldPartitionHLOD* Actor);
	void OnHLODActorUnregistered(AWorldPartitionHLOD* Actor);

	void OnLevelEditorCreated(TSharedPtr<ILevelEditor> InLevelEditor);
	void ExtendShowFlagsMenu();

	void OnPostWorldInitialization(UWorld* InWorld, const UWorld::InitializationValues InIVS);

private:
	FDelegateHandle HLODActorEditorRegisteredHandle;
	FDelegateHandle HLODActorEditorUnregisteredHandle;

	FVector CachedCameraLocation;
	double CachedHLODMinDrawDistance;
	double CachedHLODMaxDrawDistance;
	bool bCachedShowHLODsOverLoadedRegions;
	bool bForceHLODStateUpdate;
	bool bForceHLODVisibilityUpdate;
	bool bHLODSettingsFilteringActive;

	TMap<TObjectKey<UWorldPartition>, TPimplPtr<FWorldPartitionHLODEditorData>> WorldPartitionsHLODEditorData;

	typedef TMap<TSoftObjectPtr<UStruct>, TSet<FName>> FStructsPropertiesMap;
	static UE_API TMap<EHLODSettingsVisibility, FStructsPropertiesMap> StructsPropertiesVisibility;
};


// Macros to simplify registration of HLOD settings filtering
#define HLOD_ADD_CLASS_SETTING_FILTER_NAME(SettingsLevel, TypeIdentifier, PropertyName) \
	UWorldPartitionHLODEditorSubsystem::AddHLODSettingsFilter(EHLODSettingsVisibility::SettingsLevel, TypeIdentifier::StaticClass(), (PropertyName))

#define HLOD_ADD_STRUCT_SETTING_FILTER_NAME(SettingsLevel, TypeIdentifier, PropertyName) \
	UWorldPartitionHLODEditorSubsystem::AddHLODSettingsFilter(EHLODSettingsVisibility::SettingsLevel, TypeIdentifier::StaticStruct(), (PropertyName))

#define HLOD_ADD_CLASS_SETTING_FILTER(SettingsLevel, TypeIdentifier, PropertyIdentifier) \
	HLOD_ADD_CLASS_SETTING_FILTER_NAME(SettingsLevel, TypeIdentifier, GET_MEMBER_NAME_CHECKED(TypeIdentifier, PropertyIdentifier))

#define HLOD_ADD_STRUCT_SETTING_FILTER(SettingsLevel, TypeIdentifier, PropertyIdentifier) \
	HLOD_ADD_STRUCT_SETTING_FILTER_NAME(SettingsLevel, TypeIdentifier, GET_MEMBER_NAME_CHECKED(TypeIdentifier, PropertyIdentifier))

#undef UE_API
