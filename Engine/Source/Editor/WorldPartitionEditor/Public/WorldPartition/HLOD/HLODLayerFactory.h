// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"

#include "HLODLayerFactory.generated.h"

#define UE_API WORLDPARTITIONEDITOR_API

class FFeedbackContext;
class UClass;
class UObject;

UCLASS(MinimalAPI)
class UHLODLayerFactory : public UFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UFactory Interface
	UE_API virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	//~ Begin UFactory Interface	
};

#undef UE_API
