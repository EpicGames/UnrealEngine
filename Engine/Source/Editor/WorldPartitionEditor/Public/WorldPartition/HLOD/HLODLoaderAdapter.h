// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/WorldPartitionActorLoaderInterface.h"

class FHLODActorDesc;

class FLoaderAdapterHLOD : public IWorldPartitionActorLoaderInterface::ILoaderAdapter
{
public:
	FLoaderAdapterHLOD(UWorld* InWorld);

protected:
	//~ Begin ILoaderAdapter interface
	virtual void ForEachActor(TFunctionRef<void(const FWorldPartitionHandle&)> InOperation) const override;
	virtual bool PassActorDescFilter(const FWorldPartitionHandle& ActorHandle) const override;
	//~ End ILoaderAdapter interface
};
