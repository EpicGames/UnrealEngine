// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "WorldPartition/WorldPartition.h"

class SWorldPartitionEditor : public SCompoundWidget, public IWorldPartitionEditor
{
public:
	SLATE_BEGIN_ARGS(SWorldPartitionEditor)
		:_InWorld(nullptr)
		{}
		SLATE_ARGUMENT(UWorld*, InWorld)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	~SWorldPartitionEditor();

	// IWorldPartitionEditor interface
	virtual void Refresh() override;
	virtual void Reconstruct() override;
	virtual void FocusBox(const FBox& Box) const override;

private:
	void OnBrowseWorld(UWorld* InWorld);

	TSharedRef<SWidget> ConstructContentWidget();

	FText GetMouseLocationText() const;

	TSharedPtr<SBorder>	ContentParent;
	TSharedPtr<class SWorldPartitionEditorGrid> GridView;
	TWeakObjectPtr<UWorld> World;
};