// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Layout/Children.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class UWorldPartition;

#define WORLD_PARTITION_EDITOR_IMPL(Type) \
	SLATE_BEGIN_ARGS(Type) \
		:_InWorld(nullptr) \
		{} \
	SLATE_ARGUMENT(UWorld*, InWorld) \
	SLATE_END_ARGS() \
	static TSharedRef<SWorldPartitionEditorGrid> CreateInstance(TSharedPtr<SWorldPartitionEditorGrid>& InPtr, UWorld* InWorld) \
	{ \
		return SAssignNew(InPtr, Type).InWorld(InWorld).Me(); \
	}

/**
 * Base class for world partition editors (goes hand in hand with corresponding UWorldPartition class via GetWorldPartitionEditorName)
 */
class SWorldPartitionEditorGrid : public SCompoundWidget
{
	friend class SWorldPartitionEditor;

	typedef TFunction<TSharedRef<SWorldPartitionEditorGrid> (TSharedPtr<SWorldPartitionEditorGrid>&, UWorld*)> PartitionEditorGridCreateInstanceFunc;

public:
	WORLD_PARTITION_EDITOR_IMPL(SWorldPartitionEditorGrid);

	SWorldPartitionEditorGrid()
		: World(nullptr)
	{}

	void Construct(const FArguments& InArgs);

	// Interface to register world partition editors
	static void RegisterPartitionEditorGridCreateInstanceFunc(FName Name, PartitionEditorGridCreateInstanceFunc CreateFunc);
	static PartitionEditorGridCreateInstanceFunc GetPartitionEditorGridCreateInstanceFunc(FName Name);

	// SPanel interface
	virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override
	{
		if (IsDisabled())
		{
			SCompoundWidget::OnArrangeChildren(AllottedGeometry, ArrangedChildren);
		}
	}

	virtual FVector2D ComputeDesiredSize(float Scale) const override
	{
		return IsDisabled() ? FVector2D(100.0f, 100.0f) : SCompoundWidget::ComputeDesiredSize(Scale);
	}

	virtual FChildren* GetChildren() override
	{
		return IsDisabled() ? &ChildSlot : (FChildren*)&FNoChildren::NoChildrenInstance;
	}
	
	bool GetPlayerView(FVector& Location, FRotator& Rotation) const;
	bool GetObserverView(FVector& Location, FRotator& Rotation) const;

	void Refresh();
	virtual void FocusBox(const FBox& Box) const {}

	inline UWorld* GetWorld() { return World.Get(); }
	inline const UWorld* GetWorld() const { return World.Get(); }
	
	inline UWorldPartition* GetWorldPartition() { return WorldPartition.Get(); }
	inline const UWorldPartition* GetWorldPartition() const { return WorldPartition.Get(); }

protected:
	static TMap<FName, PartitionEditorGridCreateInstanceFunc> PartitionEditorGridCreateInstanceFactory;

private:
	bool IsDisabled() const;

	TWeakObjectPtr<UWorld> World;
	TWeakObjectPtr<UWorldPartition> WorldPartition;
};