// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakInterfacePtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Layout/ArrangedChildren.h"
#include "SWorldPartitionEditorGrid.h"
#include "SWorldPartitionViewportWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "SViewportToolBar.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionActorLoaderInterface.h"
#include "ExternalDirtyActorsTracker.h"

class SWorldPartitionEditorGrid2D : public SWorldPartitionEditorGrid
{
protected:
	class FEditorCommands : public TCommands<FEditorCommands>
	{
	public:
		FEditorCommands();
	
		// Context Menu
		TSharedPtr<FUICommandInfo> CreateRegionFromSelection;
		TSharedPtr<FUICommandInfo> BuildHLODsForSelection;
		TSharedPtr<FUICommandInfo> BuildHLODsForSelectedRegions;
		TSharedPtr<FUICommandInfo> LoadSelectedRegions;
		TSharedPtr<FUICommandInfo> UnloadSelectedRegions;
		TSharedPtr<FUICommandInfo> UnloadHoveredRegion;
		TSharedPtr<FUICommandInfo> ConvertSelectedRegionsToActors;
		TSharedPtr<FUICommandInfo> MoveCameraHere;
		TSharedPtr<FUICommandInfo> PlayFromHere;
		TSharedPtr<FUICommandInfo> LoadFromHere;
		TSharedPtr<FUICommandInfo> BugItHere;

		// Toolbar
		// Options
		TSharedPtr<FUICommandInfo> FollowPlayerInPIE;
		TSharedPtr<FUICommandInfo> BugItGoLoadRegion;

		// Show toggles
		TSharedPtr<FUICommandInfo> ShowActors;
		TSharedPtr<FUICommandInfo> ShowHLODActors;
		TSharedPtr<FUICommandInfo> ShowGrid;
		TSharedPtr<FUICommandInfo> ShowMiniMap;
		TSharedPtr<FUICommandInfo> ShowCoords;
		TSharedPtr<FUICommandInfo> ShowMouseCoords;
		TSharedPtr<FUICommandInfo> ShowLoadingRegions;

		// Quick Actions
		TSharedPtr<FUICommandInfo> FocusSelection;
		TSharedPtr<FUICommandInfo> FocusLoadedRegions;
		TSharedPtr<FUICommandInfo> FocusWorld;
		
		/**
		 * Initialize commands
		 */
		virtual void RegisterCommands() override;
	};

	// In-viewport toolbar widget used in the world partition editor
	class SToolBar : public SViewportToolBar
	{
	public:
		SLATE_BEGIN_ARGS(SToolBar) {}
			SLATE_ARGUMENT(SWorldPartitionEditorGrid2D*, WPEditorGrid2D)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

	private:
		EVisibility IsOptionsMenuVisible() const;
		
		TSharedRef<SWidget> GenerateUnloadedOpacitySlider() const;

		TSharedRef<SWidget> GenerateOptionsMenu() const;
		TSharedRef<SWidget> GenerateShowMenu() const;
		TSharedRef<SWidget> GenerateBuildMenu() const;
		
		TSharedPtr<FUICommandList> CommandList;
		SWorldPartitionEditorGrid2D* WPEditorGrid2D;
	};


public:
	SWorldPartitionEditorGrid2D();
	~SWorldPartitionEditorGrid2D();

protected:
	void Construct(const FArguments& InArgs);
	void BindCommands();

	void CreateRegionFromSelection();
	void BuildHLODsForSelection();
	void BuildHLODsForSelectedRegions();
	void LoadSelectedRegions();
	void UnloadSelectedRegions();
	void ConvertSelectedRegionsToActors();
	
	void MoveCameraHere();
	void PlayFromHere();
	void LoadFromHere();
	void BugItHere();

	bool IsFollowPlayerInPIE() const;
	bool IsInteractive() const;

	virtual int64 GetSelectionSnap() const;
	
	TSharedRef<SWidget> GenerateContextualMenu() const;

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual int32 PaintGrid(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;
	virtual uint32 PaintActors(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, uint32 LayerId) const;
	virtual uint32 PaintTextInfo(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, uint32 LayerId) const;
	virtual uint32 PaintViewer(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, uint32 LayerId) const;
	virtual uint32 PaintSelection(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, uint32 LayerId) const;
	virtual int32 PaintMinimap(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;
	virtual int32 PaintMeasureTool(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

	int32 DrawTextLabel(FSlateWindowElementList& OutDrawElements, int32 LayerId, const FGeometry& AllottedGeometry, const FString& Label, const FVector2D& Pos, const FLinearColor& Color, const FSlateFontInfo& Font) const;
	
	void FocusSelection();
	void FocusLoadedRegions();
	void FocusWorld();

	void UpdateTransform() const;
	void UpdateSelectionBox(bool bSnap);
	void ClearSelection();

	bool ShouldShowActorBounds(AActor* InActor) const;
	bool ShouldShowActorBounds(FWorldPartitionActorDescInstance* ActorDescInstance) const;

	const TSharedRef<FUICommandList> CommandList;

	FSingleWidgetChildrenWithBasicLayoutSlot ChildSlot;

	FChildren* GetChildren()
	{
		return &ChildSlot;
	}

	void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
	{
		ArrangedChildren.AddWidget(AllottedGeometry.MakeChild(ChildSlot.GetWidget(), FVector2D::ZeroVector, AllottedGeometry.GetLocalSize()));
	}

	virtual void FocusBox(const FBox& Box) const override;

	mutable float Scale;
	mutable FVector2D Trans;
	
	mutable FBox2D ScreenRect;
	mutable FTransform2d WorldToScreen;
	mutable FTransform2d ScreenToWorld;

	bool bIsDragSelecting;
	bool bIsPanning;
	bool bIsMeasuring;
	bool bShowActors;
	bool bShowHLODActors;
	bool bShowGrid;
	bool bShowLoadingRegions;
	bool bShowMiniMap;
	bool bShowMouseCoords;
	bool bFollowPlayerInPIE;
	FVector2D MouseCursorPos;
	FVector2D MouseCursorPosWorld;
	FVector2D LastMouseCursorPosWorldDrag;
	FVector2D SelectionStart;
	FVector2D SelectionEnd;
	FVector2D MeasureStart;
	FVector2D MeasureEnd;
	FBox SelectBox;
	FBox SelectBoxGridSnapped;
	FSlateFontInfo SmallLayoutFont;
	float TotalMouseDelta;

	struct FKeyFuncs : public BaseKeyFuncs<TWeakInterfacePtr<IWorldPartitionActorLoaderInterface>, TWeakInterfacePtr<IWorldPartitionActorLoaderInterface>, false>
	{
		static KeyInitType GetSetKey(ElementInitType Entry)
		{
			return Entry;
		}

		static bool Matches(KeyInitType A, KeyInitType B)
		{
			return A == B;
		}

		static uint32 GetKeyHash(KeyInitType Key)
		{
			return GetTypeHash(Key.GetWeakObjectPtr());
		}
	};

	using FLoaderInterface = TWeakInterfacePtr<IWorldPartitionActorLoaderInterface>;
	using FLoaderInterfaceSet = TSet<FLoaderInterface, FKeyFuncs>;
	using FLoaderInterfaceStack = TArray<FLoaderInterface>;
	
	FLoaderInterfaceSet SelectedLoaderInterfaces;
	
	// Updated every tick
	TSet<FGuid> ShownActorGuids;
	TSet<FGuid> DirtyActorGuids;
	TSet<FGuid> SelectedActorGuids;
	FLoaderInterfaceSet ShownLoaderInterfaces;
	FLoaderInterfaceSet HoveredLoaderInterfaces;
	FLoaderInterfaceStack HoveredLoaderInterfacesStack;
	FLoaderInterface HoveredLoaderInterface;

	// Shown actors cache
	FBox ViewRectWorldCache;
	TSet<FGuid> ShownActorGuidsCache;
	FLoaderInterfaceSet ShownLoaderInterfacesCache;
	void InvalidateShownActorsCache() { ViewRectWorldCache.Init(); }

	// Minimap
	void UpdateWorldMiniMapDetails();
	void SaveMiniMapUnloadedOpacityUserSetting();
	bool IsMiniMapUnloadedOpacityEnabled() const;
	float GetMiniMapUnloadedOpacity() const { return MiniMapUnloadedOpacity; };
	void SetMiniMapUnloadedOpacity(float InOpacity) { MiniMapUnloadedOpacity = InOpacity; };
	float MiniMapUnloadedOpacity;
	FBox2D WorldMiniMapBounds;
	FSlateBrush WorldMiniMapBrush;

	// Profiling
	mutable double TickTime;
	mutable double PaintTime;

	TSharedPtr<SWorldPartitionViewportWidget> ViewportWidget;
	TSharedPtr<STextBlock> TextWorldBoundsInKMWidget;
	TSharedPtr<STextBlock> TextRulerWidget;

	struct FExternalDirtyActorTrackerGuid
	{
		using Type = FGuid;
		using OwnerType = SWorldPartitionEditorGrid2D;
		static FGuid Store(SWorldPartitionEditorGrid2D* InOwner, AActor* InActor) { return InActor->GetActorGuid(); }
	};

	using FExternalDirtyActorsTracker = TExternalDirtyActorsTracker<FExternalDirtyActorTrackerGuid>;
	TUniquePtr<FExternalDirtyActorsTracker> ExternalDirtyActorsTracker;
};