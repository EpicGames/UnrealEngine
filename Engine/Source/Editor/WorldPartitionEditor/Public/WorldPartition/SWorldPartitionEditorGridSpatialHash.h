// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SWorldPartitionEditorGrid2D.h"

class SWorldPartitionEditorGridSpatialHash : public SWorldPartitionEditorGrid2D
{
public:
	WORLD_PARTITION_EDITOR_IMPL(SWorldPartitionEditorGridSpatialHash);

	void Construct(const FArguments& InArgs);

	virtual int64 GetSelectionSnap() const override;
	virtual int32 PaintGrid(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const override;

private:
	int64 GetSelectionSnap(float& FadeRatio, float& CellScreenSize) const;

	inline static float WantedCellScreenSize = 64.0f;
};
