// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Commandlets/WorldPartitionConvertCommandlet.h"
#include "WorldPartitionConvertOptions.generated.h"

UCLASS()
class UWorldPartitionConvertOptions : public UObject
{
	GENERATED_BODY()

public:
	FString ToCommandletArgs() const;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category=Convert)
	TSubclassOf<UWorldPartitionConvertCommandlet> CommandletClass;
		
	UPROPERTY(EditAnywhere, Category=Convert, meta = (ToolTip = "Wether the conversion should create a new map with a _WP suffix or overwrite the source map"))
	bool bInPlace;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Convert)
	bool bDeleteSourceLevels;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Convert)
	bool bGenerateIni;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Convert)
	bool bReportOnly;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Convert)
	bool bVerbose;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Convert)
	bool bSkipStableGUIDValidation;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Convert)
	bool bOnlyMergeSubLevels;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = Convert)
	bool bSaveFoliageTypeToContentFolder = true;

	FString LongPackageName;
};