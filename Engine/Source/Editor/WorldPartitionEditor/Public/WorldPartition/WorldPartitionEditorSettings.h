// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "Commandlets/WorldPartitionConvertCommandlet.h"
#include "WorldPartitionEditorSettings.generated.h"

#define UE_API WORLDPARTITIONEDITOR_API

UCLASS(MinimalAPI, config = EditorSettings, meta = (DisplayName = "World Partition"))
class UWorldPartitionEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UWorldPartitionEditorSettings();

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnWorldPartitionEditorSettingsChanged, const FName&, const UWorldPartitionEditorSettings&);
	static UE_API FOnWorldPartitionEditorSettingsChanged& OnSettingsChanged();

	static UE_API FName GetEnableLoadingInEditorPropertyName();
	static UE_API FName GetEnableAdvancedHLODSettingsPropertyName();

	UE_API TSubclassOf<UWorldPartitionConvertCommandlet> GetCommandletClass() const;
	UE_API void SetCommandletClass(const TSubclassOf<UWorldPartitionConvertCommandlet>& InCommandletClass);

	UE_API int32 GetInstancedFoliageGridSize() const;
	UE_API void SetInstancedFoliageGridSize(int32 InInstancedFoliageGridSize);

	UE_API int32 GetMinimapLowQualityWorldUnitsPerPixelThreshold() const;
	UE_API void SetMinimapLowQualityWorldUnitsPerPixelThreshold(int32 InMinimapLowQualityWorldUnitsPerPixelThreshold);

	UE_API bool GetEnableLoadingInEditor() const;
	UE_API void SetEnableLoadingInEditor(bool bInEnableLoadingInEditor);

	UE_API bool GetEnableStreamingGenerationLogOnPIE() const;
	UE_API void SetEnableStreamingGenerationLogOnPIE(bool bInEnableStreamingGenerationLogOnPIE);

	UE_API bool GetShowHLODsInEditor() const;
	UE_API void SetShowHLODsInEditor(bool bInShowHLODsInEditor);

	UE_API bool GetShowHLODsOverLoadedRegions() const;
	UE_API void SetShowHLODsOverLoadedRegions(bool bInShowHLODsOverLoadedRegions);

	UE_API bool GetEnableAdvancedHLODSettings() const;
	UE_API void SetEnableAdvancedHLODSettings(bool bInEnableAdvancedHLODSettings);

	UE_API double GetHLODMinDrawDistance() const;
	UE_API void SetHLODMinDrawDistance(double bInHLODMinDrawDistance);

	UE_API double GetHLODMaxDrawDistance() const;
	UE_API void SetHLODMaxDrawDistance(double bInHLODMaxDrawDistance);

	UE_API bool GetDisableBugIt() const;
	UE_API void SetDisableBugIt(bool bInEnableBugIt);

	UE_API bool GetDisablePIE() const;
	UE_API void SetDisablePIE(bool bInEnablePIE);

	UE_API bool GetAdvancedMode() const;
	UE_API void SetAdvancedMode(bool bInAdvancedMode);

public:
	UE_DEPRECATED(5.5, "Use Get/SetCommandletClass()")
	UPROPERTY(Config, EditAnywhere, Category = MapConversion, Meta = (ToolTip = "Commandlet class to use for World Partition conversion"))
	TSubclassOf<UWorldPartitionConvertCommandlet> CommandletClass;

	UE_DEPRECATED(5.5, "Use Get/SetInstancedFoliageGridSize()")
	UPROPERTY(Config, EditAnywhere, Category = Foliage, Meta = (ClampMin = 3200, ToolTip = "Editor grid size used for instance foliage actors in World Partition worlds"))
	int32 InstancedFoliageGridSize;

	UE_DEPRECATED(5.5, "Use Get/SetMinimapLowQualityWorldUnitsPerPixelThreshold()")
	UPROPERTY(Config, EditAnywhere, Category = MiniMap, Meta = (ClampMin = 100, ToolTip = "Threshold from which minimap generates a warning if its WorldUnitsPerPixel is above this value"))
	int32 MinimapLowQualityWorldUnitsPerPixelThreshold;

	UE_DEPRECATED(5.5, "Use Get/SetEnableLoadingInEditor()")
	UPROPERTY(Config, EditAnywhere, Category = WorldPartition, Meta = (ToolTip = "Whether to enable dynamic loading in the editor through loading regions"))
	bool bEnableLoadingInEditor;

	UE_DEPRECATED(5.5, "Use Get/SetEnableStreamingGenerationLogOnPIE()")
	UPROPERTY(Config, EditAnywhere, Category = WorldPartition, Meta = (ToolTip = "Whether to enable streaming generation log on PIE"))
	bool bEnableStreamingGenerationLogOnPIE;

	UE_DEPRECATED(5.5, "Use Get/SetShowHLODsInEditor()")
	UPROPERTY(Config, EditAnywhere, Category = HLOD, Meta = (ToolTip = "Whether to show HLODs in the editor"))
	bool bShowHLODsInEditor;

	UE_DEPRECATED(5.5, "Use Get/SetShowHLODsOverLoadedRegions()")
	UPROPERTY(Config, EditAnywhere, Category = HLOD, Meta = (ToolTip = "Control display of HLODs in case actors are loaded"))
	bool bShowHLODsOverLoadedRegions;

	UE_DEPRECATED(5.5, "Use Get/SetEnableAdvancedHLODSettings()")
	UPROPERTY(Config, AdvancedDisplay, Meta = (DisplayName = "Enable Advanced HLOD Settings", ToolTip = "Enable advanced HLODs settings"))
	bool bEnableAdvancedHLODSettings;

	UE_DEPRECATED(5.5, "Use Get/SetHLODMinDrawDistance()")
	UPROPERTY(Config, EditAnywhere, Category = HLOD, Meta = (UIMin = 0, UIMax = 1638400, DisplayName = "HLOD Min Draw Distance", ToolTip = "Minimum distance at which HLODs should be displayed in editor"))
	double HLODMinDrawDistance;

	UE_DEPRECATED(5.5, "Use Get/SetHLODMaxDrawDistance()")
	UPROPERTY(Config, EditAnywhere, Category = HLOD, Meta = (UIMin = 0, UIMax = 1638400, DisplayName = "HLOD Max Draw Distance", ToolTip = "Maximum distance at which HLODs should be displayed in editor"))
	double HLODMaxDrawDistance;

	UE_DEPRECATED(5.5, "Use Get/SetDisableBugIt()")
	bool bDisableBugIt;

	UE_DEPRECATED(5.5, "Use Get/SetDisablePIE()")
	bool bDisablePIE;

	UE_DEPRECATED(5.5, "Use Get/SetAdvancedMode()")
	bool bAdvancedMode;

protected:
	FOnWorldPartitionEditorSettingsChanged SettingsChangedDelegate;

	UE_API void TriggerPropertyChangeEvent(FName PropertyName);
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

};

#undef UE_API
