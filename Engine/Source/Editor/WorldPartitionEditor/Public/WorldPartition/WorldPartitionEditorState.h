// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorState/WorldEditorState.h"
#include "WorldPartitionEditorState.generated.h"

UCLASS(MinimalAPI)
class UWorldPartitionEditorState : public UWorldDependantEditorState
{
	GENERATED_UCLASS_BODY()

	//~ Begin UEditorState Interface
	virtual FText GetCategoryText() const override;
private:
	virtual FOperationResult CaptureState() override;
	virtual FOperationResult RestoreState() const override;
	//~ End UEditorState Interface

private:
	UPROPERTY(EditAnywhere, Category = WorldState)
	TArray<FBox> LoadedEditorRegions;

	UPROPERTY(EditAnywhere, Category = WorldState)
	TArray<FName> LoadedEditorLocationVolumes;
};
