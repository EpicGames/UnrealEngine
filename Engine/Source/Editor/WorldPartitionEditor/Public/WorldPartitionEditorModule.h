// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "WorldPartition/IWorldPartitionEditorModule.h"

class FTabManager;
class FLayoutExtender;
class SDockTab;
class FSpawnTabArgs;
class SContentBundleBrowser;
struct FAssetData;
struct FSceneOutlinerInitializationOptions;


/**
 * The module holding all of the UI related pieces for SubLevels management
 */
class FWorldPartitionEditorModule : public IWorldPartitionEditorModule
{
public:
	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override;

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule() override;
	
	/**
	 * Creates a world partition widget
	 */
	virtual TSharedRef<class SWidget> CreateWorldPartitionEditor();

	/**
	 * Returns placement grid size setting that should be assigned to new AWorldSettings actors.
	 */
	virtual int32 GetPlacementGridSize() const override;

	/**
	 * Returns foliage grid size setting that should be assigned to new AWorldSettings actors.
	 */
	virtual int32 GetInstancedFoliageGridSize() const override;

	/**
	 * Returns the threshold from which minimap generates a warning if its WorldUnitsPerPixel is above this value.
	 */
	virtual int32 GetMinimapLowQualityWorldUnitsPerPixelThreshold() const override;

	/**
	 * Returns if loading in the editor is enabled or not.
	 */
	virtual bool GetEnableLoadingInEditor() const override;
	virtual void SetEnableLoadingInEditor(bool bInEnableLoadingInEditor) override;

	/**
	* Returns if streaming generation log on PIE is enabled or not.
	*/
	virtual bool GetEnableStreamingGenerationLogOnPIE() const override;
	virtual void SetEnableStreamingGenerationLogOnPIE(bool bInEnableLoadingInEditor) override;

	/**
	 * Returns if pie is disabled or not.
	 */
	virtual bool GetDisablePIE() const override;
	virtual void SetDisablePIE(bool bInDisablePIE) override;

	/**
	 * Returns if bugit command in the editor is disabled or not.
	 */
	virtual bool GetDisableBugIt() const override;
	virtual void SetDisableBugIt(bool bInDisableBugIt) override;

	/**
	 * Returns if world partition is in advanced mode or not.
	 */
	virtual bool GetAdvancedMode() const override;
	virtual void SetAdvancedMode(bool bInAdvancedMode) override;

	virtual bool GetRequireExplicitHLODLayerPartitionAssignation() const override;
	virtual void SetRequireExplicitHLODLayerPartitionAssignation(bool bInRequireExplicitHLODLayerPartitionAssignation) override;

	virtual bool GetShowHLODsInEditor() const override;
	virtual void SetShowHLODsInEditor(bool bInShowHLODsInEditor) override;
 
	virtual bool GetShowHLODsOverLoadedRegions() const override;
	virtual void SetShowHLODsOverLoadedRegions(bool bInShowHLODsOverLoadedRegions) override;

	virtual double GetHLODInEditorMinDrawDistance() const override;
	virtual void SetHLODInEditorMinDrawDistance(double InMinDrawDistance) override;

	virtual double GetHLODInEditorMaxDrawDistance() const override;
	virtual void SetHLODInEditorMaxDrawDistance(double InMaxDrawDistance) override;

	virtual bool IsHLODInEditorAllowed(UWorld* InWorld, FText* OutDisallowedReason) const override;

	virtual bool HasErrors(UWorld* World) const override;
	virtual void RepairErrors(UWorld* World) override;

	/**
     * Writes various HLOD stats to a file.
     */ 
	virtual bool WriteHLODStats(const FWriteHLODStatsParams& Params) const override;

	/**
	 * Convert the specified map to a world partition map.
	 */
	virtual bool ConvertMap(const FString& InLongPackageName) override;

	/**
	 * Run a world partition builder for the current map. Will display a dialog to specify options for the generation.
	 */
	virtual bool RunBuilder(const FRunBuilderParams& InParams) override;

	/** Return the world added event. */
	virtual FWorldPartitionCreated& OnWorldPartitionCreated() override { return WorldPartitionCreatedEvent; }

	/** Return the commandlet pre-execution event */
	virtual FOnPreExecuteCommandlet& OnPreExecuteCommandlet() override { return OnPreExecuteCommandletEvent; }

	/** Return the commandlet execution event */
	virtual FOnExecuteCommandlet& OnExecuteCommandlet() override { return OnExecuteCommandletEvent; }

	/** Return the commandlet post-execution event */
	virtual FOnPostExecuteCommandlet& OnPostExecuteCommandlet() override { return OnPostExecuteCommandletEvent; }


	/**
	 * Creates a Content Bundle Browser widget
	 */
	TSharedRef<class SWidget> CreateContentBundleBrowser();

	/**
	 * Returns if there's a content bundle in editing mode.
	 */
	bool IsEditingContentBundle() const;

	/**
	 * Returns if the content bundle is in editing mode.
	 */
	bool IsEditingContentBundle(const FGuid& ContentBundleGuid) const;

	/**
	 * Get the camera location & rotation from the active level viewport.
	 */
	static bool GetActiveLevelViewportCameraInfo(FVector& CameraLocation, FRotator& CameraRotation);

private:
	/** Register menus */
	void RegisterMenus();

	/** Registers world partition tabs spawners with the level editor */
	void RegisterWorldPartitionTabs(TSharedPtr<FTabManager> InTabManager);

	/** Inserts world partition tabs into the level editor layout */
	void RegisterWorldPartitionLayout(FLayoutExtender& Extender);

	/** Spawns the world partition tab */
	TSharedRef<SDockTab> SpawnWorldPartitionTab(const FSpawnTabArgs& Args);

	/** Spawns the content bundle tab */
	TSharedRef<SDockTab> SpawnContentBundleTab(const FSpawnTabArgs& Args);

	bool Build(const FRunBuilderParams& InParams);
	bool BuildMinimap(const FRunBuilderParams& InParams);
	bool BuildHLODs(const FRunBuilderParams& InParams);
	bool BuildLandscapeSplineMeshes(UWorld* InWorld);

private:
	void RunCommandletAsExternalProcess(const FString& InCommandletArgs, const FText& InOperationDescription, int32& OutResult, bool& bOutCancelled);
	void OnConvertMap();
	bool OnIsCrossPluginReferenceAllowed(const FAssetData& ReferencingAssetData, const FAssetData& ReferencedAssetData);
	void OnCreateActorBrowserColumns(FSceneOutlinerInitializationOptions& InitOptions, UWorld* World);

	FDelegateHandle EditorInitializedHandle;
	FDelegateHandle EditorCloseHandle;
	FDelegateHandle LevelEditorExtenderDelegateHandle;

	TWeakPtr<SDockTab> WorldPartitionTab;
	TWeakPtr<SDockTab> ContentBundleTab;

	TWeakPtr<SContentBundleBrowser> ContentBundleBrowser;

	FWorldPartitionCreated WorldPartitionCreatedEvent;

	FOnPreExecuteCommandlet OnPreExecuteCommandletEvent;
	FOnExecuteCommandlet OnExecuteCommandletEvent;
	FOnPostExecuteCommandlet OnPostExecuteCommandletEvent;

	bool bRequireExplicitHLODLayerPartitionAssignation;
};
