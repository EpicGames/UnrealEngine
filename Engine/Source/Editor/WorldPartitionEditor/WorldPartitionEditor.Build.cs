// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class WorldPartitionEditor : ModuleRules
{
    public WorldPartitionEditor(ReadOnlyTargetRules Target) : base(Target)
    {     
        PrivateDependencyModuleNames.AddRange(
            new string[] {
				"ApplicationCore",
				"CommonMenuExtensions",
				"ContentBrowser",
				"ContentBrowserData",
				"Core",
				"CoreUObject",
				"DataLayerEditor",
				"DeveloperSettings",
				"EditorFramework",
				"EditorSubsystem",
				"EditorWidgets",
				"Engine",
				"ImageCore",
				"InputCore",
				"Json",
				"LevelEditor",	
				"MainFrame",
				"PropertyEditor",				
				"SceneOutliner",
				"Slate",
				"SlateCore",
				"RenderCore",
				"Renderer",
				"RHI",				
				"ToolMenus",
				"UnrealEd",
				"WorldBrowser"
			}
		);

        DynamicallyLoadedModuleNames.AddRange(
            new string[] {
				"AssetTools",
            }
		);

		PrivateIncludePathModuleNames.AddRange
		(
			new string[]
			{
				"WorkspaceMenuStructure",
			}
		);

		PrivateIncludePaths.AddRange(
			new string[] {
			}
		);
	}
}
